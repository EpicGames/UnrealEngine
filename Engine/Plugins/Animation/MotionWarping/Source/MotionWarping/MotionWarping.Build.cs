// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class MotionWarping : ModuleRules
{
	public MotionWarping(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PrivateIncludePaths.Add(Path.Combine(EngineDirectory,"Source/ThirdParty/AHEasing/AHEasing-1.3.2"));

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",		
				"Engine",
				"NetCore"
			}
			);
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Slate",
				"SlateCore",
			}
			);
		
		if (Target.bBuildEditor)
		{
			PublicDependencyModuleNames.AddRange(new string []
				{
					"UnrealEd",
					"AnimGraph",
				});
		}
	}
}
