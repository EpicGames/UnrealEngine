// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNotifyState_MotionWarping.h"
#include "MotionWarpingComponent.h"
#include "RootMotionModifier_SkewWarp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotifyState_MotionWarping)

#if WITH_EDITOR
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#endif

UAnimNotifyState_MotionWarping::UAnimNotifyState_MotionWarping(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RootMotionModifier = ObjectInitializer.CreateDefaultSubobject<URootMotionModifier_SkewWarp>(this, TEXT("RootMotionModifier_SkewWarp"));
}

void UAnimNotifyState_MotionWarping::OnBecomeRelevant(UMotionWarpingComponent* MotionWarpingComp, const UAnimSequenceBase* Animation, float StartTime, float EndTime) const
{
	URootMotionModifier* RootMotionModifierNew = AddRootMotionModifier(MotionWarpingComp, Animation, StartTime, EndTime);

	if (RootMotionModifierNew)
	{
		if (!RootMotionModifierNew->OnActivateDelegate.IsBound())
		{
			RootMotionModifierNew->OnActivateDelegate.BindDynamic(const_cast<UAnimNotifyState_MotionWarping*>(this), &UAnimNotifyState_MotionWarping::OnRootMotionModifierActivate);
		}

		if (!RootMotionModifierNew->OnUpdateDelegate.IsBound())
		{
			RootMotionModifierNew->OnUpdateDelegate.BindDynamic(const_cast<UAnimNotifyState_MotionWarping*>(this), &UAnimNotifyState_MotionWarping::OnRootMotionModifierUpdate);
		}

		if (!RootMotionModifierNew->OnDeactivateDelegate.IsBound())
		{
			RootMotionModifierNew->OnDeactivateDelegate.BindDynamic(const_cast<UAnimNotifyState_MotionWarping*>(this), &UAnimNotifyState_MotionWarping::OnRootMotionModifierDeactivate);
		}
	}
}

URootMotionModifier* UAnimNotifyState_MotionWarping::AddRootMotionModifier_Implementation(UMotionWarpingComponent* MotionWarpingComp, const UAnimSequenceBase* Animation, float StartTime, float EndTime) const
{
	if (MotionWarpingComp && RootMotionModifier)
	{
		return MotionWarpingComp->AddModifierFromTemplate(RootMotionModifier, Animation, StartTime, EndTime);
	}

	return nullptr;
}

void UAnimNotifyState_MotionWarping::OnRootMotionModifierActivate(UMotionWarpingComponent* MotionWarpingComp, URootMotionModifier* Modifier)
{
	// Notify blueprint
	OnWarpBegin(MotionWarpingComp, Modifier);
}

void UAnimNotifyState_MotionWarping::OnRootMotionModifierUpdate(UMotionWarpingComponent* MotionWarpingComp, URootMotionModifier* Modifier)
{
	// Notify blueprint
	OnWarpUpdate(MotionWarpingComp, Modifier);
}

void UAnimNotifyState_MotionWarping::OnRootMotionModifierDeactivate(UMotionWarpingComponent* MotionWarpingComp, URootMotionModifier* Modifier)
{
	// Notify blueprint
	OnWarpEnd(MotionWarpingComp, Modifier);
}

#if WITH_EDITOR
void UAnimNotifyState_MotionWarping::ValidateAssociatedAssets()
{
	static const FName NAME_AssetCheck("AssetCheck");

	if(UAnimSequenceBase* ContainingAsset = Cast<UAnimSequenceBase>(GetContainingAsset()))
	{
		if (RootMotionModifier == nullptr)
		{
			FMessageLog AssetCheckLog(NAME_AssetCheck);

			const FText MessageLooping = FText::Format(
				NSLOCTEXT("AnimNotify", "MotionWarping_InvalidRootMotionModifier", "Motion Warping window in {0} doesn't have a valid RootMotionModifier"),
				FText::AsCultureInvariant(GetNameSafe(ContainingAsset)));
			AssetCheckLog.Warning()
				->AddToken(FUObjectToken::Create(ContainingAsset))
				->AddToken(FTextToken::Create(MessageLooping));

			if (GIsEditor)
			{
				const bool bForce = true;
				AssetCheckLog.Notify(MessageLooping, EMessageSeverity::Warning, bForce);
			}
		}
		else if (const URootMotionModifier_Warp* RootMotionModifierWarp = Cast<URootMotionModifier_Warp>(RootMotionModifier))
		{
			if (RootMotionModifierWarp->WarpTargetName.IsNone())
			{
				FMessageLog AssetCheckLog(NAME_AssetCheck);

				const FText MessageLooping = FText::Format(
					NSLOCTEXT("AnimNotify", "MotionWarping_InvalidWarpTargetName", "Motion Warping window in {0} doesn't specify a valid Warp Target Name"),
					FText::AsCultureInvariant(GetNameSafe(ContainingAsset)));
				AssetCheckLog.Warning()
					->AddToken(FUObjectToken::Create(ContainingAsset))
					->AddToken(FTextToken::Create(MessageLooping));

				if (GIsEditor)
				{
					const bool bForce = true;
					AssetCheckLog.Notify(MessageLooping, EMessageSeverity::Warning, bForce);
				}
			}
		}
	}
}

void UAnimNotifyState_MotionWarping::DrawInEditor(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* MeshComp, const UAnimSequenceBase* Animation, const FAnimNotifyEvent& NotifyEvent) const
{
	if (RootMotionModifier)
	{
		FAnimNotifyEvent NotifyEventWithColor = NotifyEvent;
		NotifyEventWithColor.NotifyColor = NotifyColor;
		
		// Necessary for FCompactPose (likely used by most RootMotionModifiers) that uses a FAnimStackAllocator (TMemStackAllocator) which allocates from FMemStack.
		// When allocating memory from FMemStack we need to explicitly use FMemMark to ensure items are freed when the scope exits. 
		FMemMark Mark(FMemStack::Get());
		RootMotionModifier->DrawInEditor(PDI, MeshComp, Animation, NotifyEventWithColor);
	}
}

void UAnimNotifyState_MotionWarping::DrawCanvasInEditor(FCanvas& Canvas, FSceneView& View, USkeletalMeshComponent* MeshComp, const UAnimSequenceBase* Animation, const FAnimNotifyEvent& NotifyEvent) const
{
	if (RootMotionModifier)
	{
		FAnimNotifyEvent NotifyEventWithColor = NotifyEvent;
		NotifyEventWithColor.NotifyColor = NotifyColor;

		// Necessary for FCompactPose (likely used by most RootMotionModifiers) that uses a FAnimStackAllocator (TMemStackAllocator) which allocates from FMemStack.
		// When allocating memory from FMemStack we need to explicitly use FMemMark to ensure items are freed when the scope exits.
		FMemMark Mark(FMemStack::Get());
		RootMotionModifier->DrawCanvasInEditor(Canvas, View, MeshComp, Animation, NotifyEventWithColor);
	}
}

#endif
