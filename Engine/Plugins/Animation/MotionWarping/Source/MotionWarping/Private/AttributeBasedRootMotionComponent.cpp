// Copyright Epic Games, Inc. All Rights Reserved.

#include "AttributeBasedRootMotionComponent.h"

#include "Animation/AnimInstance.h"
#include "Animation/AnimationPoseData.h"
#include "BonePose.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "Components/SkeletalMeshComponent.h"
#include "GameFramework/Character.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AttributeBasedRootMotionComponent)

// UAttributeBasedRootMotionComponent
///////////////////////////////////////////////////////////////////////

UAttributeBasedRootMotionComponent::UAttributeBasedRootMotionComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bWantsInitializeComponent = true;
	PrimaryComponentTick.TickGroup = TG_PostUpdateWork;
	PrimaryComponentTick.bCanEverTick = true;
}

void UAttributeBasedRootMotionComponent::InitializeComponent()
{
	Super::InitializeComponent();

	CharacterOwner = Cast<ACharacter>(GetOwner());
	
	ACharacter* Character = GetCharacterOwner();
	check(Character);
	
	PrePhysicsTickFunction.bCanEverTick = true;
	PrePhysicsTickFunction.bStartWithTickEnabled = true;
	PrePhysicsTickFunction.SetTickFunctionEnable(true);
	PrePhysicsTickFunction.TickGroup = TG_PrePhysics;
	PrePhysicsTickFunction.Target = this;
}

void UAttributeBasedRootMotionComponent::RegisterComponentTickFunctions(bool bRegister)
{
	Super::RegisterComponentTickFunctions(bRegister);

	if (Mode == EAttributeBasedRootMotionMode::ApplyVelocity)
	{
		if (bRegister)
		{
			CharacterOwner = Cast<ACharacter>(GetOwner());
			ACharacter* Character = GetCharacterOwner();
			check(Character);
			UCharacterMovementComponent* CharacterMovement = Character->GetCharacterMovement<UCharacterMovementComponent>();

			PrePhysicsTickFunction.RegisterTickFunction(GetComponentLevel());
			CharacterMovement->PrimaryComponentTick.AddPrerequisite(this, PrePhysicsTickFunction);
		}
		else
		{
			PrePhysicsTickFunction.UnRegisterTickFunction();
		}
	}
}


void UAttributeBasedRootMotionComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	if (bEnableRootMotion)
	{
		static const FName RootMotionAttributeName = "RootMotionDelta";
		static const UE::Anim::FAttributeId RootMotionAttributeId = { RootMotionAttributeName , FCompactPoseBoneIndex(0) };
	
		Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	
		if (ACharacter* Character = GetCharacterOwner())
		{
			if (const USkeletalMeshComponent* Mesh = Character->GetMesh())
			{
				if (const FTransformAnimationAttribute* RootMotionAttribute = Mesh->GetCustomAttributes().Find<FTransformAnimationAttribute>(RootMotionAttributeId))
				{
					TranslationVelocity = RootMotionAttribute->Value.GetTranslation() / DeltaTime;

					const FQuat RootMotionRotation = RootMotionAttribute->Value.GetRotation().GetShortestArcWith(FQuat::Identity);
					RotationVelocity = RootMotionRotation.ToRotationVector() / DeltaTime;

					if (Mode == EAttributeBasedRootMotionMode::ApplyDelta)
					{
						if (UCharacterMovementComponent* CharacterMovement = Character->GetCharacterMovement<UCharacterMovementComponent>())
						{
							CharacterMovement->RootMotionParams.Set(RootMotionAttribute->Value);
						}
					}
				
					UE_VLOG_ARROW(this, "Root Motion", Display, Character->GetActorTransform().GetLocation(), Character->GetActorTransform().GetLocation() + Mesh->GetComponentTransform().TransformVector(TranslationVelocity) * 0.1, FColor::Green, TEXT(""));
				}
			}
		}
	}
}


void FAttributeBasedRootMotionComponentPrePhysicsTickFunction::ExecuteTick(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	FActorComponentTickFunction::ExecuteTickHelper(Target, /*bTickInEditor=*/ false, DeltaTime, TickType, [this](float DilatedTime)
		{
			Target->PrePhysicsTickComponent(DilatedTime);
		});
}

FString FAttributeBasedRootMotionComponentPrePhysicsTickFunction::DiagnosticMessage()
{
	return Target->GetFullName() + TEXT("[UAttributeBasedRootMotionComponent::PrePhysicsTick]");
}

FName FAttributeBasedRootMotionComponentPrePhysicsTickFunction::DiagnosticContext(bool bDetailed)
{
	if (bDetailed)
	{
		return FName(*FString::Printf(TEXT("AttributeBasedRootMotionComponentPrePhysicsTick/%s"), *GetFullNameSafe(Target)));
	}

	return FName(TEXT("AttributeBasedRootMotionComponentPrePhysicsTick"));
}

void UAttributeBasedRootMotionComponent::PrePhysicsTickComponent(float DeltaTime)
{
	if (!bEnableRootMotion)
	{
		return;
	}
	
	if (ACharacter* Character = GetCharacterOwner())
	{
		if (UCharacterMovementComponent* CharacterMovement = Character->GetCharacterMovement<UCharacterMovementComponent>())
		{
			FTransform RootMotion;
			RootMotion.SetTranslation(TranslationVelocity * DeltaTime);
			RootMotion.SetRotation(FQuat::MakeFromRotationVector(RotationVelocity*DeltaTime));
			CharacterMovement->RootMotionParams.Set(RootMotion);
		}
	}
}
