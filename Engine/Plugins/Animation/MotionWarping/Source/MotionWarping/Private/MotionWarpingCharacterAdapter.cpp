// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotionWarpingCharacterAdapter.h"
#include "GameFramework/Character.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "Components/CapsuleComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MotionWarpingCharacterAdapter)



void UMotionWarpingCharacterAdapter::BeginDestroy()
{
	const ACharacter* RawTargetCharacter = TargetCharacter.Get();
	UCharacterMovementComponent* CharacterMovement = RawTargetCharacter ? RawTargetCharacter->GetCharacterMovement() : nullptr;
	if (CharacterMovement)
	{
		CharacterMovement->ProcessRootMotionPreConvertToWorld.Unbind();
	}

	Super::BeginDestroy();
}

void UMotionWarpingCharacterAdapter::SetCharacter(ACharacter* InCharacter)
{
	if (ensureMsgf(InCharacter && InCharacter->GetCharacterMovement(), TEXT("Invalid Character or missing CharacterMovementComponent. Motion warping will not function.")))
	{
		TargetCharacter = InCharacter;
		InCharacter->GetCharacterMovement()->ProcessRootMotionPreConvertToWorld.BindUObject(this, &UMotionWarpingCharacterAdapter::WarpLocalRootMotionOnCharacter);
	}
}

AActor* UMotionWarpingCharacterAdapter::GetActor() const
{ 
	return Cast<AActor>(TargetCharacter.Get());
}

USkeletalMeshComponent* UMotionWarpingCharacterAdapter::GetMesh() const
{ 
	if (const ACharacter* RawTargetCharacter = TargetCharacter.Get())
	{
		return RawTargetCharacter->GetMesh();
	}
	return nullptr;
}

FVector UMotionWarpingCharacterAdapter::GetVisualRootLocation() const
{
	if (const ACharacter* RawTargetCharacter = TargetCharacter.Get())
	{
		const float CapsuleHalfHeight = RawTargetCharacter->GetCapsuleComponent()->GetScaledCapsuleHalfHeight();
		const FQuat CurrentRotation = RawTargetCharacter->GetActorQuat();
		return  (RawTargetCharacter->GetActorLocation() - CurrentRotation.GetUpVector() * CapsuleHalfHeight);
	}
	return FVector::ZeroVector;
}

FVector UMotionWarpingCharacterAdapter::GetBaseVisualTranslationOffset() const
{
	if (const ACharacter* RawTargetCharacter = TargetCharacter.Get())
	{
		return RawTargetCharacter->GetBaseTranslationOffset();
	}
	return FVector::ZeroVector;
}

FQuat UMotionWarpingCharacterAdapter::GetBaseVisualRotationOffset() const
{ 
	if (const ACharacter* RawTargetCharacter = TargetCharacter.Get())
	{
		return RawTargetCharacter->GetBaseRotationOffset();
	}
	return FQuat::Identity;
}

FTransform UMotionWarpingCharacterAdapter::WarpLocalRootMotionOnCharacter(const FTransform& LocalRootMotionTransform, UCharacterMovementComponent* TargetMoveComp, float DeltaSeconds)
{
	const ACharacter* RawTargetCharacter = TargetCharacter.Get();
	if (WarpLocalRootMotionDelegate.IsBound() && RawTargetCharacter)
	{
		FMotionWarpingUpdateContext WarpingContext;
		
		WarpingContext.DeltaSeconds = DeltaSeconds;

		// When replaying saved moves we need to look at the contributor to root motion back then.
		if (RawTargetCharacter->bClientUpdating)
		{
			const UCharacterMovementComponent* MoveComp = RawTargetCharacter->GetCharacterMovement();
			check(MoveComp);

			const FSavedMove_Character* SavedMove = MoveComp->GetCurrentReplayedSavedMove();
			check(SavedMove);

			if (SavedMove->RootMotionMontage.IsValid())
			{
				WarpingContext.Animation = SavedMove->RootMotionMontage.Get();
				WarpingContext.CurrentPosition = SavedMove->RootMotionTrackPosition;
				WarpingContext.PreviousPosition = SavedMove->RootMotionPreviousTrackPosition;
				WarpingContext.PlayRate = SavedMove->RootMotionPlayRateWithScale;
			}
		}
		else // If we are not replaying a move, just use the current root motion montage
		{
			if (const FAnimMontageInstance* RootMotionMontageInstance = RawTargetCharacter->GetRootMotionAnimMontageInstance())
			{
				const UAnimMontage* Montage = RootMotionMontageInstance->Montage;
				check(Montage);

				WarpingContext.Animation = Montage;
				WarpingContext.CurrentPosition = RootMotionMontageInstance->GetPosition();
				WarpingContext.PreviousPosition = RootMotionMontageInstance->GetPreviousPosition();
				WarpingContext.Weight = RootMotionMontageInstance->GetWeight();
				WarpingContext.PlayRate = RootMotionMontageInstance->Montage->RateScale * RootMotionMontageInstance->GetPlayRate();
			}
		}

		// TODO: Consider simply having a pointer to the MWComponent whereby we can call a function on it, rather than using this delegate approach
		return WarpLocalRootMotionDelegate.Execute(LocalRootMotionTransform, DeltaSeconds, &WarpingContext);
	}

	return LocalRootMotionTransform;
}
