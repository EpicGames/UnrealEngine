// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotionWarpingComponent.h"

#include "Animation/AnimMontage.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimationPoseData.h"
#include "BonePose.h"
#include "Components/SkeletalMeshComponent.h"
#include "DrawDebugHelpers.h"
#include "Engine/World.h"
#include "GameFramework/Character.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "AnimNotifyState_MotionWarping.h"
#include "MotionWarpingCharacterAdapter.h"
#include "MotionWarpingMontageTrajectoryAdapter.h"
#include "MotionWarpingSwitchOffCondition.h"
#include "Net/Core/PushModel/PushModel.h"
#include "Net/UnrealNetwork.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MotionWarpingComponent)

DEFINE_LOG_CATEGORY(LogMotionWarping);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
TAutoConsoleVariable<int32> FMotionWarpingCVars::CVarMotionWarpingDisable(TEXT("a.MotionWarping.Disable"), 0, TEXT("Disable Motion Warping"), ECVF_Cheat);
TAutoConsoleVariable<int32> FMotionWarpingCVars::CVarMotionWarpingDebug(TEXT("a.MotionWarping.Debug"), 0, TEXT("0: Disable, 1: Only Log, 2: Only DrawDebug, 3: Log and DrawDebug"), ECVF_Cheat);
TAutoConsoleVariable<float> FMotionWarpingCVars::CVarMotionWarpingDrawDebugDuration(TEXT("a.MotionWarping.DrawDebugLifeTime"), 1.f, TEXT("Time in seconds each draw debug persists.\nRequires 'a.MotionWarping.Debug 2'"), ECVF_Cheat);
TAutoConsoleVariable<int32> FMotionWarpingCVars::CVarWarpedTargetDebug(TEXT("a.MotionWarping.Debug.Target"), false, TEXT("Shows warp target debug. 0 - disabled, 1 - enabled for selected actor, 2 - enabled for all actors"), ECVF_Cheat);
TAutoConsoleVariable<int32> FMotionWarpingCVars::CVarWarpedSwitchOffConditionDebug(TEXT("a.MotionWarping.Debug.SwitchOffCondition"), false, TEXT("Shows switch off condition debug. 0 - disabled, 1 - enabled for selected actor, 2 - enabled for all actors"), ECVF_Cheat);
#endif

// UMotionWarpingUtilities
///////////////////////////////////////////////////////////////////////

void UMotionWarpingUtilities::ExtractLocalSpacePose(const UAnimSequenceBase* Animation, const FBoneContainer& BoneContainer, float Time, bool bExtractRootMotion, FCompactPose& OutPose)
{
	OutPose.SetBoneContainer(&BoneContainer);

	FBlendedCurve Curve;
	Curve.InitFrom(BoneContainer);

	FAnimExtractContext Context(static_cast<double>(Time), bExtractRootMotion);

	UE::Anim::FStackAttributeContainer Attributes;
	FAnimationPoseData AnimationPoseData(OutPose, Curve, Attributes);
	if (const UAnimSequence* AnimSequence = Cast<UAnimSequence>(Animation))
	{
		AnimSequence->GetBonePose(AnimationPoseData, Context);
	}
	else if (const UAnimMontage* AnimMontage = Cast<UAnimMontage>(Animation))
	{
		const FAnimTrack& AnimTrack = AnimMontage->SlotAnimTracks[0].AnimTrack;
		AnimTrack.GetAnimationPose(AnimationPoseData, Context);
	}
}

void UMotionWarpingUtilities::ExtractComponentSpacePose(const UAnimSequenceBase* Animation, const FBoneContainer& BoneContainer, float Time, bool bExtractRootMotion, FCSPose<FCompactPose>& OutPose)
{
	FCompactPose Pose;
	ExtractLocalSpacePose(Animation, BoneContainer, Time, bExtractRootMotion, Pose);
	OutPose.InitPose(MoveTemp(Pose));
}

FTransform UMotionWarpingUtilities::ExtractRootMotionFromAnimation(const UAnimSequenceBase* Animation, float StartTime, float EndTime)
{
	if (const UAnimMontage* Anim = Cast<UAnimMontage>(Animation))
	{
		// This is identical to UAnimMontage::ExtractRootMotionFromTrackRange and UAnimCompositeBase::ExtractRootMotionFromTrack but ignoring bEnableRootMotion
		// so we can extract root motion from the montage even if that flag is set to false in the AnimSequence(s)

		FRootMotionMovementParams AccumulatedRootMotionParams;

		if (Anim->SlotAnimTracks.Num() > 0)
		{
			const FAnimTrack& RootMotionAnimTrack = Anim->SlotAnimTracks[0].AnimTrack;

			TArray<FRootMotionExtractionStep> RootMotionExtractionSteps;
			RootMotionAnimTrack.GetRootMotionExtractionStepsForTrackRange(RootMotionExtractionSteps, StartTime, EndTime);

			for (const FRootMotionExtractionStep& CurStep : RootMotionExtractionSteps)
			{
				if (CurStep.AnimSequence)
				{
					AccumulatedRootMotionParams.Accumulate(CurStep.AnimSequence->ExtractRootMotionFromRange(CurStep.StartPosition, CurStep.EndPosition, FAnimExtractContext()));
				}
			}
		}

		return AccumulatedRootMotionParams.GetRootMotionTransform();
	}

	if (const UAnimSequence* Anim = Cast<UAnimSequence>(Animation))
	{
		return Anim->ExtractRootMotionFromRange(StartTime, EndTime, FAnimExtractContext());
	}

	return FTransform::Identity;
}

FTransform UMotionWarpingUtilities::ExtractRootTransformFromAnimation(const UAnimSequenceBase* Animation, float Time)
{
	if (const UAnimMontage* AnimMontage = Cast<UAnimMontage>(Animation))
	{
		if(const FAnimSegment* Segment = AnimMontage->SlotAnimTracks[0].AnimTrack.GetSegmentAtTime(Time))
		{
			if (const UAnimSequence* AnimSequence = Cast<UAnimSequence>(Segment->GetAnimReference()))
			{
				const float AnimSequenceTime = Segment->ConvertTrackPosToAnimPos(Time);
				return AnimSequence->ExtractRootTrackTransform(FAnimExtractContext(static_cast<double>(AnimSequenceTime)), nullptr);
			}	
		}
	}
	else if (const UAnimSequence* AnimSequence = Cast<UAnimSequence>(Animation))
	{
		return AnimSequence->ExtractRootTrackTransform(FAnimExtractContext(static_cast<double>(Time)), nullptr);
	}

	return FTransform::Identity;
}

void UMotionWarpingUtilities::GetMotionWarpingWindowsFromAnimation(const UAnimSequenceBase* Animation, TArray<FMotionWarpingWindowData>& OutWindows)
{
	if(Animation)
	{
		OutWindows.Reset();

		for (int32 Idx = 0; Idx < Animation->Notifies.Num(); Idx++)
		{
			const FAnimNotifyEvent& NotifyEvent = Animation->Notifies[Idx];
			if (UAnimNotifyState_MotionWarping* Notify = Cast<UAnimNotifyState_MotionWarping>(NotifyEvent.NotifyStateClass))
			{
				FMotionWarpingWindowData Data;
				Data.AnimNotify = Notify;
				Data.StartTime = NotifyEvent.GetTriggerTime();
				Data.EndTime = NotifyEvent.GetEndTriggerTime();
				OutWindows.Add(Data);
			}
		}
	}
}

void UMotionWarpingUtilities::GetMotionWarpingWindowsForWarpTargetFromAnimation(const UAnimSequenceBase* Animation, FName WarpTargetName, TArray<FMotionWarpingWindowData>& OutWindows)
{
	if (Animation && WarpTargetName != NAME_None)
	{
		OutWindows.Reset();

		for (int32 Idx = 0; Idx < Animation->Notifies.Num(); Idx++)
		{
			const FAnimNotifyEvent& NotifyEvent = Animation->Notifies[Idx];
			if (UAnimNotifyState_MotionWarping* Notify = Cast<UAnimNotifyState_MotionWarping>(NotifyEvent.NotifyStateClass))
			{
				if (const URootMotionModifier_Warp* Modifier = Cast<const URootMotionModifier_Warp>(Notify->RootMotionModifier))
				{
					if(Modifier->WarpTargetName == WarpTargetName)
					{
						FMotionWarpingWindowData Data;
						Data.AnimNotify = Notify;
						Data.StartTime = NotifyEvent.GetTriggerTime();
						Data.EndTime = NotifyEvent.GetEndTriggerTime();
						OutWindows.Add(Data);
					}
				}
			}
		}
	}
}


FTransform UMotionWarpingUtilities::CalculateRootTransformRelativeToWarpPointAtTime(const ACharacter& Character, const UAnimSequenceBase* Animation, float Time, const FName& WarpPointBoneName)
{
	if (const USkeletalMeshComponent* Mesh = Character.GetMesh())
	{
		if (const UAnimInstance* AnimInstance = Mesh->GetAnimInstance())
		{
			const FBoneContainer& FullBoneContainer = AnimInstance->GetRequiredBones();
			const int32 BoneIndex = FullBoneContainer.GetPoseBoneIndexForBoneName(WarpPointBoneName);
			if (BoneIndex != INDEX_NONE)
			{
				TArray<FBoneIndexType> RequiredBoneIndexArray = { 0, (FBoneIndexType)BoneIndex };
				FullBoneContainer.GetReferenceSkeleton().EnsureParentsExistAndSort(RequiredBoneIndexArray);

				FBoneContainer LimitedBoneContainer(RequiredBoneIndexArray, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::DisallowAll), *FullBoneContainer.GetAsset());

				FCSPose<FCompactPose> Pose;
				UMotionWarpingUtilities::ExtractComponentSpacePose(Animation, LimitedBoneContainer, Time, false, Pose);

				// Inverse of mesh's relative rotation. Used to convert root and warp point in the animation from Y forward to X forward
				const FTransform MeshCompRelativeRotInverse = FTransform(Character.GetBaseRotationOffset().Inverse());

				const FTransform RootTransform = MeshCompRelativeRotInverse * Pose.GetComponentSpaceTransform(FCompactPoseBoneIndex(0));
				const FTransform WarpPointTransform = MeshCompRelativeRotInverse * Pose.GetComponentSpaceTransform(FCompactPoseBoneIndex(1));
				return RootTransform.GetRelativeTransform(WarpPointTransform);
			}
		}
	}

	return FTransform::Identity;
}


FTransform UMotionWarpingUtilities::CalculateRootTransformRelativeToWarpPointAtTime(const UMotionWarpingBaseAdapter& WarpingAdapter, const UAnimSequenceBase* Animation, float Time, const FName& WarpPointBoneName)
{
	if (const USkeletalMeshComponent* Mesh = WarpingAdapter.GetMesh())
	{
		if (const UAnimInstance* AnimInstance = Mesh->GetAnimInstance())
		{
			const FBoneContainer& FullBoneContainer = AnimInstance->GetRequiredBones();
			const int32 BoneIndex = FullBoneContainer.GetPoseBoneIndexForBoneName(WarpPointBoneName);
			if (BoneIndex != INDEX_NONE)
			{
				TArray<FBoneIndexType> RequiredBoneIndexArray = { 0, (FBoneIndexType)BoneIndex };
				FullBoneContainer.GetReferenceSkeleton().EnsureParentsExistAndSort(RequiredBoneIndexArray);

				FBoneContainer LimitedBoneContainer(RequiredBoneIndexArray, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::DisallowAll), *FullBoneContainer.GetAsset());

				FCSPose<FCompactPose> Pose;
				UMotionWarpingUtilities::ExtractComponentSpacePose(Animation, LimitedBoneContainer, Time, false, Pose);

				// Inverse of mesh's relative rotation. Used to convert root and warp point in the animation from Y forward to X forward
				const FTransform MeshCompRelativeRotInverse = FTransform(WarpingAdapter.GetBaseVisualRotationOffset().Inverse());

				const FTransform RootTransform = MeshCompRelativeRotInverse * Pose.GetComponentSpaceTransform(FCompactPoseBoneIndex(0));
				const FTransform WarpPointTransform = MeshCompRelativeRotInverse * Pose.GetComponentSpaceTransform(FCompactPoseBoneIndex(1));
				return RootTransform.GetRelativeTransform(WarpPointTransform);
			}
		}
	}

	return FTransform::Identity;
}


FTransform UMotionWarpingUtilities::CalculateRootTransformRelativeToWarpPointAtTime(const ACharacter& Character, const UAnimSequenceBase* Animation, float Time, const FTransform& WarpPointTransform)
{
	// Inverse of mesh's relative rotation. Used to convert root and warp point in the animation from Y forward to X forward
	const FTransform MeshCompRelativeRotInverse = FTransform(Character.GetBaseRotationOffset().Inverse());
	const FTransform RootTransform = MeshCompRelativeRotInverse * UMotionWarpingUtilities::ExtractRootTransformFromAnimation(Animation, Time);
	return RootTransform.GetRelativeTransform((MeshCompRelativeRotInverse * WarpPointTransform));
}


FTransform UMotionWarpingUtilities::CalculateRootTransformRelativeToWarpPointAtTime(const UMotionWarpingBaseAdapter& WarpingAdapter, const UAnimSequenceBase* Animation, float Time, const FTransform& WarpPointTransform)
{
	// Inverse of mesh's relative rotation. Used to convert root and warp point in the animation from Y forward to X forward
	const FTransform MeshCompRelativeRotInverse = FTransform(WarpingAdapter.GetBaseVisualRotationOffset().Inverse());
	const FTransform RootTransform = MeshCompRelativeRotInverse * UMotionWarpingUtilities::ExtractRootTransformFromAnimation(Animation, Time);
	return RootTransform.GetRelativeTransform((MeshCompRelativeRotInverse * WarpPointTransform));
}

void UMotionWarpingUtilities::ExtractBoneTransformFromAnimationAtTime(const UAnimInstance* AnimInstance, const UAnimSequenceBase* Animation, float Time, bool bExtractRootMotion, FName BoneName, bool bLocalSpace, FTransform& OutTransform)
{
	OutTransform = FTransform::Identity;

	if (AnimInstance && Animation)
	{
		FMemMark Mark(FMemStack::Get());

		const int32 MeshBoneIndex = AnimInstance->GetRequiredBones().GetPoseBoneIndexForBoneName(BoneName);
		const FCompactPoseBoneIndex PoseBoneIndex = AnimInstance->GetRequiredBones().MakeCompactPoseIndex(FMeshPoseBoneIndex(MeshBoneIndex));

		if (PoseBoneIndex != INDEX_NONE)
		{
			if (bLocalSpace)
			{
				FCompactPose Pose;
				ExtractLocalSpacePose(Animation, AnimInstance->GetRequiredBones(), Time, bExtractRootMotion, Pose);
				OutTransform = Pose[PoseBoneIndex];
			}
			else
			{
				FCSPose<FCompactPose> Pose;
				UMotionWarpingUtilities::ExtractComponentSpacePose(Animation, AnimInstance->GetRequiredBones(), Time, bExtractRootMotion, Pose);
				OutTransform = Pose.GetComponentSpaceTransform(PoseBoneIndex);
			}
		}
	}
}

// UMotionWarpingComponent
///////////////////////////////////////////////////////////////////////

UMotionWarpingComponent::UMotionWarpingComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bWantsInitializeComponent = true;
	SetIsReplicatedByDefault(true);
}

void UMotionWarpingComponent::GetLifetimeReplicatedProps(TArray< FLifetimeProperty >& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	FDoRepLifetimeParams Params;
	Params.bIsPushBased = true;
	Params.Condition = COND_SimulatedOnly;
	DOREPLIFETIME_WITH_PARAMS_FAST(UMotionWarpingComponent, WarpTargets, Params);
}

void UMotionWarpingComponent::InitializeComponent()
{
	Super::InitializeComponent();

	// Implicitly support Characters if no other adapter has already been setup
	if (GetOwnerAdapter() == nullptr)
	{
		if (ACharacter* CharacterOwner = Cast<ACharacter>(GetOwner()))
		{
			UMotionWarpingCharacterAdapter* CharacterAdapter = CreateOwnerAdapter<UMotionWarpingCharacterAdapter>();
			CharacterAdapter->SetCharacter(CharacterOwner);
		}
	}
}

UMotionWarpingBaseAdapter* UMotionWarpingComponent::CreateOwnerAdapter(TSubclassOf<UMotionWarpingBaseAdapter> AdapterClass)
{
	check(AdapterClass);	
	OwnerAdapter = NewObject<UMotionWarpingBaseAdapter>(this, AdapterClass);
	OwnerAdapter->WarpLocalRootMotionDelegate.BindUObject(this, &UMotionWarpingComponent::ProcessRootMotionPreConvertToWorld);

	return OwnerAdapter;
}

bool UMotionWarpingComponent::ContainsModifier(const UAnimSequenceBase* Animation, float StartTime, float EndTime) const
{
	return Modifiers.ContainsByPredicate([=](const URootMotionModifier* Modifier)
		{
			return (Modifier->Animation == Animation && Modifier->StartTime == StartTime && Modifier->EndTime == EndTime);
		});
}

int32 UMotionWarpingComponent::AddModifier(URootMotionModifier* Modifier)
{
	if (ensureAlways(Modifier))
	{
		UE_LOGF(LogMotionWarping, Verbose, "MotionWarping: RootMotionModifier added. NetMode: %d WorldTime: %f Char: %ls Animation: %ls [%f %f] [%f %f] Loc: %ls Rot: %ls",
			GetWorld()->GetNetMode(), GetWorld()->GetTimeSeconds(), *GetNameSafe(GetOwner()), *GetNameSafe(Modifier->Animation.Get()), Modifier->StartTime, Modifier->EndTime, Modifier->PreviousPosition, Modifier->CurrentPosition,
			*GetOwner()->GetActorLocation().ToString(), *GetOwner()->GetActorRotation().ToCompactString());

		return Modifiers.Add(Modifier);
	}

	return INDEX_NONE;
}

void UMotionWarpingComponent::DisableAllRootMotionModifiers()
{
	if (Modifiers.Num() > 0)
	{
		for (URootMotionModifier* Modifier : Modifiers)
		{
			Modifier->SetState(ERootMotionModifierState::Disabled);
		}
	}
}

void UMotionWarpingComponent::UpdateSwitchOffConditions()
{
	for (int32 i = WarpTargets.Num() - 1; i >= 0; --i)
	{
		FSwitchOffConditionData* SwitchOffConditionData = FindSwitchOffConditionData(WarpTargets[i].Name);
		if (!SwitchOffConditionData)
		{
			continue;
		}

		TArray<TObjectPtr<UMotionWarpingSwitchOffCondition>>* Conditions = &SwitchOffConditionData->SwitchOffConditions;
		if (!Conditions)
		{
			continue;
		}

		bool bClearCondition = false;
		bool bPauseWarping = false;
		bool bPauseRootMotion = false;

		for (const UMotionWarpingSwitchOffCondition* Condition : *Conditions)
		{
			if (!IsValid(Condition) || !Condition->IsConditionValid())
			{
				continue;
			}
			if (Condition->Check())
			{
				switch (Condition->GetEffect())
				{
				case ESwitchOffConditionEffect::CancelFollow:
					if (WarpTargets[i].bFollowComponent)
					{
						WarpTargets[i].bFollowComponent = false;
						WarpTargets[i].Location = Condition->GetTargetLocation();
						WarpTargets[i].Rotation = Condition->GetTargetRotation();
					}
					break;
				case ESwitchOffConditionEffect::CancelWarping:
					bClearCondition = true;
					break;
				case ESwitchOffConditionEffect::PauseWarping:
					bPauseWarping = true;
					break;
				case ESwitchOffConditionEffect::PauseRootMotion:
					bPauseRootMotion = true;
					break;
				default:
					checkNoEntry();
				}
			}
		}

		//remove finished and invalid conditions
		if (bClearCondition)
		{
			RemoveSwitchOffConditions(WarpTargets[i].Name);
			WarpTargets.RemoveAtSwap(i);
		}
		else
		{
			WarpTargets[i].bWarpingPaused = bPauseWarping;
			WarpTargets[i].bRootMotionPaused = bPauseRootMotion;
		}
	}
}

void UMotionWarpingComponent::UpdateWithContext(const FMotionWarpingUpdateContext& Context, float DeltaSeconds)
{
	UpdateSwitchOffConditions();

	// Currently just uses existing modifiers and swaps state, no need to create
	// Also We can predict outside of gamethread which means creating Modifiers / UObjects.
	if (!bPredictingTrajectory && Context.Animation.IsValid())
	{
		const UAnimSequenceBase* Animation = Context.Animation.Get();
		const float PreviousPosition = Context.PreviousPosition;
		const float CurrentPosition = Context.CurrentPosition;

		// Loop over notifies directly in the montage, looking for Motion Warping windows
		for (const FAnimNotifyEvent& NotifyEvent : Animation->Notifies)
		{
			const UAnimNotifyState_MotionWarping* MotionWarpingNotify = NotifyEvent.NotifyStateClass ? Cast<UAnimNotifyState_MotionWarping>(NotifyEvent.NotifyStateClass) : nullptr;
			if (MotionWarpingNotify)
			{
				if(MotionWarpingNotify->RootMotionModifier == nullptr)
				{
					UE_LOGF(LogMotionWarping, Warning, "MotionWarpingComponent::Update. A motion warping window in %ls doesn't have a valid root motion modifier!", *GetNameSafe(Animation));
					continue;
				}

				const float StartTime = FMath::Clamp(NotifyEvent.GetTriggerTime(), 0.f, Animation->GetPlayLength());
				const float EndTime = FMath::Clamp(NotifyEvent.GetEndTriggerTime(), 0.f, Animation->GetPlayLength());

				if (PreviousPosition >= StartTime && PreviousPosition < EndTime)
				{
					if (!ContainsModifier(Animation, StartTime, EndTime))
					{
						MotionWarpingNotify->OnBecomeRelevant(this, Animation, StartTime, EndTime);
					}
				}
			}
		}

		if(bSearchForWindowsInAnimsWithinMontages)
		{
			if(const UAnimMontage* Montage = Cast<const UAnimMontage>(Context.Animation.Get()))
			{
				// Same as before but scanning all animation within the montage
				for (int32 SlotIdx = 0; SlotIdx < Montage->SlotAnimTracks.Num(); SlotIdx++)
				{
					const FAnimTrack& AnimTrack = Montage->SlotAnimTracks[SlotIdx].AnimTrack;

					if (const FAnimSegment* AnimSegment = AnimTrack.GetSegmentAtTime(PreviousPosition))
					{
						if (const UAnimSequenceBase* AnimReference = AnimSegment->GetAnimReference())
						{
							for (const FAnimNotifyEvent& NotifyEvent : AnimReference->Notifies)
							{
								const UAnimNotifyState_MotionWarping* MotionWarpingNotify = NotifyEvent.NotifyStateClass ? Cast<UAnimNotifyState_MotionWarping>(NotifyEvent.NotifyStateClass) : nullptr;
								if (MotionWarpingNotify)
								{
									if (MotionWarpingNotify->RootMotionModifier == nullptr)
									{
										UE_LOGF(LogMotionWarping, Warning, "MotionWarpingComponent::Update. A motion warping window in %ls doesn't have a valid root motion modifier!", *GetNameSafe(AnimReference));
										continue;
									}

									const float NotifyStartTime = FMath::Clamp(NotifyEvent.GetTriggerTime(), 0.f, AnimReference->GetPlayLength());
									const float NotifyEndTime = FMath::Clamp(NotifyEvent.GetEndTriggerTime(), 0.f, AnimReference->GetPlayLength());

									// Convert notify times from AnimSequence times to montage times
									const float StartTime = (NotifyStartTime - AnimSegment->AnimStartTime) + AnimSegment->StartPos;
									const float EndTime = (NotifyEndTime - AnimSegment->AnimStartTime) + AnimSegment->StartPos;

									if (PreviousPosition >= StartTime && PreviousPosition < EndTime)
									{
										if (!ContainsModifier(Montage, StartTime, EndTime))
										{
											MotionWarpingNotify->OnBecomeRelevant(this, Montage, StartTime, EndTime);
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}

	OnPreUpdate.Broadcast(this);

	// Update the state of all the modifiers
	if (Modifiers.Num() > 0)
	{
		for (URootMotionModifier* Modifier : Modifiers)
		{
			Modifier->Update(Context);
		}

		// Remove the modifiers that has been marked for removal
		Modifiers.RemoveAll([this](const URootMotionModifier* Modifier)
		{
			if (Modifier->GetState() == ERootMotionModifierState::MarkedForRemoval)
			{
				UE_LOGF(LogMotionWarping, Verbose, "MotionWarping: RootMotionModifier removed. NetMode: %d WorldTime: %f Char: %ls Animation: %ls [%f %f] [%f %f] Loc: %ls Rot: %ls",
					GetWorld()->GetNetMode(), GetWorld()->GetTimeSeconds(), *GetNameSafe(GetOwner()), *GetNameSafe(Modifier->Animation.Get()), Modifier->StartTime, Modifier->EndTime, Modifier->PreviousPosition, Modifier->CurrentPosition,
					*GetOwner()->GetActorLocation().ToString(), *GetOwner()->GetActorRotation().ToCompactString());

				return true;
			}

			return false;
		});
	}
}

FTransform UMotionWarpingComponent::ProcessRootMotionPreConvertToWorld(const FTransform& InRootMotion, float DeltaSeconds, const FMotionWarpingUpdateContext* InContext)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (FMotionWarpingCVars::CVarMotionWarpingDisable.GetValueOnAnyThread() > 0)
	{
		return InRootMotion;
	}
#endif
	if (!InContext)
	{
		return InRootMotion;
	}

	// Check for warping windows and update modifier states
	UpdateWithContext(*InContext, DeltaSeconds);

	FTransform FinalRootMotion = InRootMotion;

	// Apply Local Space Modifiers
	for (URootMotionModifier* Modifier : Modifiers)
	{
		if (Modifier->GetState() == ERootMotionModifierState::Active)
		{
			FinalRootMotion = Modifier->ProcessRootMotion(FinalRootMotion, DeltaSeconds);
		}
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	const int32 DebugLevel = FMotionWarpingCVars::CVarMotionWarpingDebug.GetValueOnAnyThread();
	if (DebugLevel >= 2 && OwnerAdapter)
	{
		const float DrawDebugDuration = FMotionWarpingCVars::CVarMotionWarpingDrawDebugDuration.GetValueOnGameThread();
		const float PointSize = 7.f;
		const FVector ActorFeetLocation = OwnerAdapter->GetVisualRootLocation();
		if (Modifiers.Num() > 0)
		{
			if (!OriginalRootMotionAccum.IsSet())
			{
				OriginalRootMotionAccum = ActorFeetLocation;
				WarpedRootMotionAccum = ActorFeetLocation;
			}
			
			OriginalRootMotionAccum = OriginalRootMotionAccum.GetValue() + (OwnerAdapter->GetMesh()->ConvertLocalRootMotionToWorld(FTransform(InRootMotion.GetLocation()))).GetLocation();
			WarpedRootMotionAccum = WarpedRootMotionAccum.GetValue() + (OwnerAdapter->GetMesh()->ConvertLocalRootMotionToWorld(FTransform(FinalRootMotion.GetLocation()))).GetLocation();

			DrawDebugPoint(GetWorld(), OriginalRootMotionAccum.GetValue(), PointSize, FColor::Red, false, DrawDebugDuration, 0);
			DrawDebugPoint(GetWorld(), WarpedRootMotionAccum.GetValue(), PointSize, FColor::Green, false, DrawDebugDuration, 0);
		}
		else
		{
			OriginalRootMotionAccum.Reset();
			WarpedRootMotionAccum.Reset();
		}

		DrawDebugPoint(GetWorld(), ActorFeetLocation, PointSize, FColor::Blue, false, DrawDebugDuration, 0);
	}

	const int32 DebugValSwitchOffCondition = FMotionWarpingCVars::CVarWarpedSwitchOffConditionDebug->GetInt();

	const bool bDebugSwitchOffCondition = (DebugValSwitchOffCondition == 1 && GetOwner()->IsSelected()) || DebugValSwitchOffCondition == 2;

	const int32 DebugValTarget = FMotionWarpingCVars::CVarWarpedTargetDebug->GetInt();
	const bool bDebugTarget = (DebugValTarget == 1 && GetOwner()->IsSelected()) || DebugValTarget == 2;

	const FVector ActorLocation = GetOwner()->GetActorLocation();
	FVector TextLocation = ActorLocation;
	constexpr float VerticalTextOffset = -10.0f;

	TArray<URootMotionModifier_Warp*> WarpModifiers;
	for (URootMotionModifier* Modifier : Modifiers)
	{
		if (Modifier->GetState() == ERootMotionModifierState::Active)
		{
			if (URootMotionModifier_Warp* WarpModifier = Cast<URootMotionModifier_Warp>(Modifier))
			{
				WarpModifiers.Add(WarpModifier);
			}
		}
	}

	for (int32 i = 0; i < WarpTargets.Num(); ++i)
	{
		const FMotionWarpingTarget& WarpingTarget = WarpTargets[i];

		// Skip inactive warp targets
		if (!WarpModifiers.ContainsByPredicate([&WarpingTarget](const URootMotionModifier_Warp* Mod) {return Mod->WarpTargetName == WarpingTarget.Name; }))
		{
			continue;
		}

		// Cycle between colors for better readability
		FColor WarpTargetColor = (i % 2) != 0 ? FColor(21, 76, 190) : FColor::Green;

		if (bDebugTarget)
		{
			FVector TargetLocation = WarpingTarget.GetTargetTrasform().GetLocation();
			DrawDebugSphere(GetWorld(), TargetLocation, 5.0f, 16, WarpTargetColor, false);

			DrawDebugLine(GetWorld(), TextLocation, WarpingTarget.GetTargetTrasform().GetLocation(), WarpTargetColor);

			FString DebugText = FString::Printf(TEXT("Warp target name: %s, is dynamic: %s"), *WarpingTarget.Name.ToString(), WarpingTarget.bFollowComponent ? TEXT("True") : TEXT("False"));
			DrawDebugString(GetWorld(), TextLocation, DebugText, nullptr, WarpTargetColor, 0.f, true);
			TextLocation.Z += VerticalTextOffset;
		}

		if (bDebugSwitchOffCondition)
		{
			if (FSwitchOffConditionData* ConditionData = FindSwitchOffConditionData(WarpingTarget.Name))
			{
				DrawDebugString(GetWorld(), TextLocation, TEXT("Switch off conditions:"), nullptr, FColor::White, 0.f, true);
				TextLocation.Z += VerticalTextOffset;

				for (const UMotionWarpingSwitchOffCondition* Condition : ConditionData->SwitchOffConditions)
				{
					const bool bConditionTrue = Condition->Check();
					FColor SwitchOffConditionTextColor = bConditionTrue ? FColor::Red : FColor::Yellow;
					DrawDebugString(GetWorld(),
						TextLocation,
						FString::Printf(TEXT("%s - %s"), *Condition->ExtraDebugInfo(), bConditionTrue ? TEXT("TRUE") : TEXT("FALSE")),
						nullptr,
						SwitchOffConditionTextColor,
						0.f,
						true);
					TextLocation.Z += VerticalTextOffset;
				}
			}
		}

		TextLocation.Z += VerticalTextOffset;
	}
#endif

	return FinalRootMotion;
}


bool UMotionWarpingComponent::FindAndUpdateWarpTarget(const FMotionWarpingTarget& WarpTarget)
{
	for (int32 Idx = 0; Idx < WarpTargets.Num(); Idx++)
	{
		if (WarpTargets[Idx].Name == WarpTarget.Name)
		{
			WarpTargets[Idx] = WarpTarget;
			return true;
		}
	}

	return false;
}

void UMotionWarpingComponent::AddOrUpdateWarpTarget(const FMotionWarpingTarget& WarpTarget)
{
	if (WarpTarget.Name != NAME_None)
	{
		// if we did not find the target, add it
		if (!FindAndUpdateWarpTarget(WarpTarget))
		{
			int32 Idx = WarpTargets.Add(WarpTarget);

			if (FSwitchOffConditionData* SwitchOffConditionData = FindSwitchOffConditionData(WarpTarget.Name))
			{
				SwitchOffConditionData->SetMotionWarpingTarget(&WarpTargets[Idx]);
			}
		}

		if (!bPredictingTrajectory)
		{
			MARK_PROPERTY_DIRTY_FROM_NAME(UMotionWarpingComponent, WarpTargets, this);
		}
	}
}

int32 UMotionWarpingComponent::RemoveWarpTarget(FName WarpTargetName)
{
	const int32 NumRemoved = WarpTargets.RemoveAll([&WarpTargetName](const FMotionWarpingTarget& WarpTarget) { return WarpTarget.Name == WarpTargetName; });
	
	if (NumRemoved > 0 && !bPredictingTrajectory)
	{
		MARK_PROPERTY_DIRTY_FROM_NAME(UMotionWarpingComponent, WarpTargets, this);
	}

	RemoveSwitchOffConditions(WarpTargetName);

	return NumRemoved;
}

int32 UMotionWarpingComponent::RemoveWarpTargets(const TArray<FName>& WarpTargetNames)
{
	int32 NumRemoved = 0;
	for (const FName& WarpTargetName : WarpTargetNames)
	{
		NumRemoved += RemoveWarpTarget(WarpTargetName);		
	}

	if (NumRemoved > 0 && !bPredictingTrajectory)
	{
		MARK_PROPERTY_DIRTY_FROM_NAME(UMotionWarpingComponent, WarpTargets, this);
	}

	return NumRemoved;
}

void UMotionWarpingComponent::AddSwitchOffCondition(FName WarpTargetName, UMotionWarpingSwitchOffCondition* Condition)
{
	if (IsValid(Condition))
	{
		if (const FMotionWarpingTarget* MotionWarpingTarget = FindWarpTarget(WarpTargetName))
		{
			Condition->SetMotionWarpingTarget(MotionWarpingTarget);
		}
		
		if (FSwitchOffConditionData* SwitchOffConditionData = FindSwitchOffConditionData(WarpTargetName))
		{
			SwitchOffConditionData->SwitchOffConditions.Add(Condition);
		}
		else
		{
			SwitchOffConditions.Add(FSwitchOffConditionData(WarpTargetName, Condition));
		}
	}
	else
	{
		UE_LOGF(LogMotionWarping, Error, "Trying to add invalid switch off condition")
	}
}

void UMotionWarpingComponent::RemoveSwitchOffConditions(FName WarpTargetName)
{
	const int32 Index = SwitchOffConditions.IndexOfByPredicate([WarpTargetName](FSwitchOffConditionData Condition)
	{
		return Condition.WarpTargetName == WarpTargetName;
	});

	if (Index != INDEX_NONE)
	{
		SwitchOffConditions.RemoveAtSwap(Index);
	}	
}

FSwitchOffConditionData* UMotionWarpingComponent::FindSwitchOffConditionData(FName WarpTargetName)
{
	return SwitchOffConditions.FindByPredicate([WarpTargetName](FSwitchOffConditionData Condition)
	{
		return Condition.WarpTargetName == WarpTargetName;
	});
}

void UMotionWarpingComponent::BeginTrajectoryPredictions(const FAnimMontageInstance* MontageInstance)
{
	check(!bPredictingTrajectory);

	// @TODO: We don't support SwitchOffConditions yet, TBD if we will
	ensure(SwitchOffConditions.IsEmpty());

	bPredictingTrajectory = true;

	CachedOwnerAdapter = OwnerAdapter;

	UMotionWarpingMontageTrajectoryAdapter* MontageTrajectoryAdapter = nullptr;
	if (!PredictionAdapter)
	{
		// Create prediction adapter and assign as owner adapter
		MontageTrajectoryAdapter = CreateOwnerAdapter<UMotionWarpingMontageTrajectoryAdapter>();
		PredictionAdapter = MontageTrajectoryAdapter;
	}
	else
	{
		OwnerAdapter = PredictionAdapter;
		MontageTrajectoryAdapter = CastChecked<UMotionWarpingMontageTrajectoryAdapter>(OwnerAdapter);
	}

	MontageTrajectoryAdapter->InitializeFrom(CachedOwnerAdapter, MontageInstance);
}

void UMotionWarpingComponent::EndTrajectoryPredictions()
{
	check(bPredictingTrajectory);

	// Restore Pre-prediction owner
	OwnerAdapter = CachedOwnerAdapter;

	// Restore Warp State to pre-prediction
	if (UMotionWarpingMontageTrajectoryAdapter* MontageTrajectoryAdapter = Cast<UMotionWarpingMontageTrajectoryAdapter>(PredictionAdapter))
	{
		MontageTrajectoryAdapter->RestoreState();
	}

	bPredictingTrajectory = false;
}

bool UMotionWarpingComponent::IsPredictingTrajectory() const
{
	return bPredictingTrajectory;
}

int32 UMotionWarpingComponent::RemoveAllWarpTargets()
{
	const int32 NumRemoved = WarpTargets.Num();

	for (const FMotionWarpingTarget& Target : WarpTargets)
	{
		RemoveSwitchOffConditions(Target.Name);
	}
	
	WarpTargets.Reset();

	if (NumRemoved > 0 && !bPredictingTrajectory)
	{
		MARK_PROPERTY_DIRTY_FROM_NAME(UMotionWarpingComponent, WarpTargets, this);
	}

	return NumRemoved;
}

void UMotionWarpingComponent::AddOrUpdateWarpTargetFromTransform(FName WarpTargetName, FTransform TargetTransform)
{
	AddOrUpdateWarpTarget(FMotionWarpingTarget(WarpTargetName, TargetTransform));
}

void UMotionWarpingComponent::AddOrUpdateWarpTargetFromComponent(FName WarpTargetName, const USceneComponent* Component, FName BoneName, bool bFollowComponent, FVector LocationOffset, FRotator RotationOffset)
{
	AddOrUpdateWarpTargetFromComponent(WarpTargetName, Component, BoneName, bFollowComponent, EWarpTargetLocationOffsetDirection::TargetsForwardVector, LocationOffset, RotationOffset);
}

void UMotionWarpingComponent::AddOrUpdateWarpTargetFromComponent(FName WarpTargetName, const USceneComponent* Component, FName BoneName, bool bFollowComponent, EWarpTargetLocationOffsetDirection LocationOffsetDirection, FVector LocationOffset, FRotator RotationOffset)
{
	if (Component == nullptr)
	{
		UE_LOGF(LogMotionWarping, Warning, "AddOrUpdateWarpTargetFromComponent has failed!. Reason: Invalid Component");
		return;
	}

	AddOrUpdateWarpTarget(FMotionWarpingTarget(WarpTargetName, Component, BoneName, bFollowComponent, LocationOffsetDirection, GetOwner(), LocationOffset, RotationOffset));
}

URootMotionModifier* UMotionWarpingComponent::AddModifierFromTemplate(URootMotionModifier* Template, const UAnimSequenceBase* Animation, float StartTime, float EndTime)
{
	if (ensureAlways(Template))
	{
		FObjectDuplicationParameters Params(Template, this);
		URootMotionModifier* NewRootMotionModifier = CastChecked<URootMotionModifier>(StaticDuplicateObjectEx(Params));
		
		NewRootMotionModifier->Animation = Animation;
		NewRootMotionModifier->StartTime = StartTime;
		NewRootMotionModifier->EndTime = EndTime;

		AddModifier(NewRootMotionModifier);

		return NewRootMotionModifier;
	}

	return nullptr;
}
