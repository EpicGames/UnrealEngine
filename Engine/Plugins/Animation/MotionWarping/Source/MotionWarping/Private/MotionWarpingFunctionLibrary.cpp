// Copyright Epic Games, Inc. All Rights Reserved.
#include "MotionWarpingFunctionLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MotionWarpingFunctionLibrary)

FMotionWarpingTarget UMotionWarpingFunctionLibrary::MakeMotionWarpingTarget(const FName Name, const FVector Location, const FRotator Rotation, const USceneComponent* Component, FName BoneName, bool bFollowComponent, EWarpTargetLocationOffsetDirection LocationOffsetDirection, const AActor* AvatarActor, const FVector LocationOffset, const FRotator RotationOffset)
{
	if (Component)
	{
		return FMotionWarpingTarget(Name, Component, BoneName, bFollowComponent, LocationOffsetDirection, AvatarActor, LocationOffset, RotationOffset);
	}
	else
	{
		FMotionWarpingTarget Result = FMotionWarpingTarget();

		// Only certain arguments are valid when a component isn't specified
		Result.Name = Name;
		Result.Location = Location;
		Result.Rotation = Rotation;
		Result.Component = nullptr;
		Result.BoneName = NAME_None;
		Result.bFollowComponent = false;

		return Result;
	}
}
