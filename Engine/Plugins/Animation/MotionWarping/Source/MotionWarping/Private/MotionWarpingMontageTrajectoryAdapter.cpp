// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotionWarpingMontageTrajectoryAdapter.h"

#include "Animation/AnimMontage.h"
#include "Components/CapsuleComponent.h"
#include "MotionWarpingComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MotionWarpingMontageTrajectoryAdapter)

void UMotionWarpingMontageTrajectoryAdapter::InitializeFrom(const UMotionWarpingBaseAdapter* InAdapter, const FAnimMontageInstance* InMontageInstance)
{
	ensure(InAdapter);
	ensure(InMontageInstance);

	if (!InAdapter || !InMontageInstance)
	{
		return;
	}

	CurrentTransform = InAdapter->GetCurrentTransform();
	VisualRootLocation = InAdapter->GetVisualRootLocation();
	BaseVisualTranslationOffset = InAdapter->GetBaseVisualTranslationOffset();
	BaseVisualRotationOffset = InAdapter->GetBaseVisualRotationOffset();

	// @TODO: Weight can vary, technically we need to predict that. But so far nothing uses this but adjustment blend warp
	Montage = InMontageInstance->Montage;
	MontageWeight = InMontageInstance->GetWeight();
	MontagePlayRate = InMontageInstance->Montage->RateScale * InMontageInstance->GetPlayRate();

	// Cache Warp State & Create a copy of warp state for the warping component to predict with
	{
		UMotionWarpingComponent* WarpingComponent = CastChecked<UMotionWarpingComponent>(GetOuter());

		// We don't support Switch Off Conditions yet (The class itself needs to be updated to be actorless & predict both src / target
		ensure(WarpingComponent->SwitchOffConditions.IsEmpty());

		CachedWarpState.Modifiers = WarpingComponent->Modifiers;
		CachedWarpState.WarpTargets = WarpingComponent->WarpTargets;
		CachedWarpState.SwitchOffConditions = WarpingComponent->SwitchOffConditions;

		CachedWarpState.ModifierSwapStates.Reset();
		for (URootMotionModifier* Modifer : WarpingComponent->Modifiers)
		{
			// Can't duplicate object on worker thread, cache state instead
			CachedWarpState.ModifierSwapStates.Add(Modifer->GetSwapState());
		}
	}
}

void UMotionWarpingMontageTrajectoryAdapter::RestoreState()
{
	UMotionWarpingComponent* WarpingComponent = CastChecked<UMotionWarpingComponent>(GetOuter());

	WarpingComponent->Modifiers = MoveTemp(CachedWarpState.Modifiers);
	WarpingComponent->WarpTargets = MoveTemp(CachedWarpState.WarpTargets);
	WarpingComponent->SwitchOffConditions = MoveTemp(CachedWarpState.SwitchOffConditions);

	ensure(WarpingComponent->Modifiers.Num() == CachedWarpState.ModifierSwapStates.Num());
	for (int32 Index = 0; Index < CachedWarpState.ModifierSwapStates.Num(); Index++)
	{
		WarpingComponent->Modifiers[Index]->RestoreFromSwapState(CachedWarpState.ModifierSwapStates[Index]);
	}

	CachedWarpState.Modifiers.Reset();
	CachedWarpState.WarpTargets.Reset();
	CachedWarpState.SwitchOffConditions.Reset();
	CachedWarpState.ModifierSwapStates.Reset();
}

FTransform UMotionWarpingMontageTrajectoryAdapter::GetCurrentTransform() const
{
	return CurrentTransform;
}

FVector UMotionWarpingMontageTrajectoryAdapter::GetVisualRootLocation() const
{
	return VisualRootLocation;
}

void UMotionWarpingMontageTrajectoryAdapter::SetCurrentTransform(const FTransform InCurrentTransform)
{
	CurrentTransform = InCurrentTransform;
}

void UMotionWarpingMontageTrajectoryAdapter::SetVisualRootLocation(const FVector InVisualRootLocation)
{
	VisualRootLocation = InVisualRootLocation;
}

FVector UMotionWarpingMontageTrajectoryAdapter::GetBaseVisualTranslationOffset() const
{
	return BaseVisualTranslationOffset;
}

FQuat UMotionWarpingMontageTrajectoryAdapter::GetBaseVisualRotationOffset() const
{ 
	return BaseVisualRotationOffset;
}

FTransform UMotionWarpingMontageTrajectoryAdapter::WarpLocalRootMotionOnMontageTrajectory(const FTransform& LocalRootMotionTransform, float DeltaSeconds, float InMontagePreviousPosition, float InMontageCurrentPosition)
{
	if (WarpLocalRootMotionDelegate.IsBound() && Montage)
	{
		FMotionWarpingUpdateContext WarpingContext;
		
		WarpingContext.DeltaSeconds = DeltaSeconds;
		WarpingContext.Animation = Montage;
		WarpingContext.CurrentPosition = InMontageCurrentPosition;
		WarpingContext.PreviousPosition = InMontagePreviousPosition;
		WarpingContext.Weight = MontageWeight;
		WarpingContext.PlayRate = MontagePlayRate;

		return WarpLocalRootMotionDelegate.Execute(LocalRootMotionTransform, DeltaSeconds, &WarpingContext);
	}

	return LocalRootMotionTransform;
}
