// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotionWarping/Public/MotionWarpingSwitchOffCondition.h"
#include "MotionWarpingComponent.h"
#include "GameFramework/Actor.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MotionWarpingSwitchOffCondition)

void FSwitchOffConditionData::SetMotionWarpingTarget(const FMotionWarpingTarget* MotionWarpingTarget)
{
	for (UMotionWarpingSwitchOffCondition* SwitchOffCondition : SwitchOffConditions)
	{
		SwitchOffCondition->SetMotionWarpingTarget(MotionWarpingTarget);
	}
}

bool UMotionWarpingSwitchOffCondition::Check() const
{
	return OnCheck();
}

FVector UMotionWarpingSwitchOffCondition::GetTargetLocation() const
{
	if (bUseWarpTargetAsTargetLocation)
	{
		if (MotionWarpingTarget)
		{
			return MotionWarpingTarget->GetLocation();
		}

		UE_LOGF(LogMotionWarping, Warning, "Switch off condition is set to use warp target as target location, "
			"however warp target appears to be null. Make sure warp target is correctly passed, "
			"otherwise switch off condition will use target actor location.")
	}

	return TargetActor->GetActorLocation();
}

FRotator UMotionWarpingSwitchOffCondition::GetTargetRotation() const
{
	if (bUseWarpTargetAsTargetLocation)
	{
		if (MotionWarpingTarget)
		{
			return MotionWarpingTarget->Rotator();
		}

		UE_LOGF(LogMotionWarping, Warning, "Switch off condition is set to use warp target as target rotation, "
			"however warp target appears to be null. Make sure warp target is correctly passed, "
			"otherwise switch off condition will use target actor rotation.")
	}

	return TargetActor->GetActorRotation();
}

void UMotionWarpingSwitchOffCondition::SetWarpTargetForDestination(const FMotionWarpingTarget* InMotionWarpingTarget)
{
	MotionWarpingTarget = InMotionWarpingTarget;
}

bool UMotionWarpingSwitchOffCondition::IsConditionValid() const
{
	if (!IsValid(OwnerActor))
	{
		
		UE_LOGF(LogMotionWarping, Display, "%ls won't work due to invalid Owner Actor", *GetClass()->GetName())
		return false;
	}
	
	if (bUseWarpTargetAsTargetLocation)
	{
		if (!MotionWarpingTarget)
		{

			UE_LOGF(LogMotionWarping, Display, "%ls is set to use Motion Warping Target as target location, but won't work due to null Motion Warping Target", *GetClass()->GetName())
			return false;
		}

		return true;
	}

	if (!IsValid(TargetActor))
	{
		UE_LOGF(LogMotionWarping, Display, "%ls on actor %ls is set to use Actor as target location, but won't work due to invalid Target Actor", *GetClass()->GetName(), *OwnerActor->GetName());
		return  false;
	}
	
	return true;
}

UMotionWarpingSwitchOffDistanceCondition* UMotionWarpingSwitchOffDistanceCondition::CreateSwitchOffDistanceCondition(AActor* InOwnerActor, ESwitchOffConditionEffect InEffect, ESwitchOffConditionDistanceOp InOperator, ESwitchOffConditionDistanceAxesType InAxesType, float InDistance, bool InbUseWarpTargetAsTargetLocation, AActor* InTargetActor)
{
	UMotionWarpingSwitchOffDistanceCondition* SwitchOffDistanceCondition = NewObject<UMotionWarpingSwitchOffDistanceCondition>();
	SwitchOffDistanceCondition->OwnerActor = InOwnerActor;
	SwitchOffDistanceCondition->Effect = InEffect;
	SwitchOffDistanceCondition->Operator = InOperator;
	SwitchOffDistanceCondition->AxesType = InAxesType;
	SwitchOffDistanceCondition->Distance = InDistance;
	SwitchOffDistanceCondition->bUseWarpTargetAsTargetLocation = InbUseWarpTargetAsTargetLocation;
	SwitchOffDistanceCondition->TargetActor = InTargetActor;

	return SwitchOffDistanceCondition;
}

bool UMotionWarpingSwitchOffDistanceCondition::OnCheck() const
{
	switch (AxesType)
	{
		case ESwitchOffConditionDistanceAxesType::AllAxes:
			return Operator == ESwitchOffConditionDistanceOp::LessThan ?
				CalculateSqDistance() < FMath::Square(Distance) :
				CalculateSqDistance() > FMath::Square(Distance);
		
		case ESwitchOffConditionDistanceAxesType::IgnoreZAxis:
			return Operator == ESwitchOffConditionDistanceOp::LessThan ?
				CalculateSqDistance2D() < FMath::Square(Distance) :
				CalculateSqDistance2D() > FMath::Square(Distance);
		
		case ESwitchOffConditionDistanceAxesType::OnlyZAxis:
			return Operator == ESwitchOffConditionDistanceOp::LessThan ?
				CalculateZDistance() < Distance :
				CalculateZDistance() > Distance;
	}

	return false;
}

float UMotionWarpingSwitchOffDistanceCondition::CalculateSqDistance() const
{
	return (OwnerActor->GetActorLocation() - GetTargetLocation()).SquaredLength();
}

float UMotionWarpingSwitchOffDistanceCondition::CalculateSqDistance2D() const
{
	return (OwnerActor->GetActorLocation() - GetTargetLocation()).SizeSquared2D();
}

float UMotionWarpingSwitchOffDistanceCondition::CalculateZDistance() const
{
	return FMath::Abs(OwnerActor->GetActorLocation().Z - GetTargetLocation().Z);
}

FString UMotionWarpingSwitchOffDistanceCondition::ExtraDebugInfo() const
{
	switch (AxesType)
	{
		case ESwitchOffConditionDistanceAxesType::AllAxes:
			return FString::Printf(TEXT("Distance: %f %c %f"),
				FMath::Sqrt(CalculateSqDistance()),
				(Operator == ESwitchOffConditionDistanceOp::GreaterThan ? '>' : '<'),
				Distance);
		
		case ESwitchOffConditionDistanceAxesType::IgnoreZAxis:
			return FString::Printf(TEXT("Distance2D: %f %c %f"),
				FMath::Sqrt(CalculateSqDistance2D()),
				(Operator == ESwitchOffConditionDistanceOp::GreaterThan ? '>' : '<'),
				Distance);
		
		case ESwitchOffConditionDistanceAxesType::OnlyZAxis:
			return FString::Printf(TEXT("Distance Z: %f %c %f"),
				CalculateZDistance(),
				(Operator == ESwitchOffConditionDistanceOp::GreaterThan ? '>' : '<'),
				Distance);
	}

	return FString();
}

UMotionWarpingSwitchOffAngleToTargetCondition* UMotionWarpingSwitchOffAngleToTargetCondition::CreateSwitchOffAngleToTargetCondition(AActor* InOwnerActor, ESwitchOffConditionEffect InEffect, ESwitchOffConditionAngleOp InOperator, float InAngle, bool bInIgnoreZAxis, bool bInUseWarpTargetAsTargetLocation, AActor* InTargetActor)
{
	UMotionWarpingSwitchOffAngleToTargetCondition* SwitchOffAngleToTargetCondition = NewObject<UMotionWarpingSwitchOffAngleToTargetCondition>();
	SwitchOffAngleToTargetCondition->OwnerActor = InOwnerActor;
	SwitchOffAngleToTargetCondition->Effect = InEffect;
	SwitchOffAngleToTargetCondition->Operator = InOperator;
	SwitchOffAngleToTargetCondition->Angle = InAngle;
	SwitchOffAngleToTargetCondition->bUseWarpTargetAsTargetLocation = bInUseWarpTargetAsTargetLocation;
	SwitchOffAngleToTargetCondition->TargetActor = InTargetActor;

	return SwitchOffAngleToTargetCondition;
}

bool UMotionWarpingSwitchOffAngleToTargetCondition::OnCheck() const
{
	if (Operator == ESwitchOffConditionAngleOp::LessThan)
	{
		return CalculateAngleToTarget() < Angle;
	}

	return CalculateAngleToTarget() > Angle;
}

FString UMotionWarpingSwitchOffAngleToTargetCondition::ExtraDebugInfo() const
{
	return FString::Printf(TEXT("Angle: %f %c %f"),
		CalculateAngleToTarget(),
		(Operator == ESwitchOffConditionAngleOp::GreaterThan ? '>' : '<'),
		Angle);
}

float UMotionWarpingSwitchOffAngleToTargetCondition::CalculateAngleToTarget() const
{
	FVector OwnerForward = OwnerActor->GetActorForwardVector();
	FVector OwnerToTarget = GetTargetLocation() - OwnerActor->GetActorLocation();

	if (bIgnoreZAxis)
	{
		OwnerForward = FVector(OwnerForward.X, OwnerForward.Y, 0);
		OwnerToTarget = FVector(OwnerToTarget.X, OwnerToTarget.Y, 0);
		OwnerForward.Normalize();
	}

	OwnerToTarget.Normalize();

	return FMath::RadiansToDegrees(FMath::Acos(FVector::DotProduct(OwnerForward, OwnerToTarget)));
}

void UMotionWarpingSwitchOffCompositeCondition::SetOwnerActor(const AActor* InOwnerActor)
{
	Super::SetOwnerActor(InOwnerActor);
	
	SwitchOffConditionA->SetOwnerActor(InOwnerActor);
	SwitchOffConditionB->SetOwnerActor(InOwnerActor);
}

void UMotionWarpingSwitchOffCompositeCondition::SetTargetActor(const AActor* InTargetActor)
{
	Super::SetTargetActor(InTargetActor);
	
	SwitchOffConditionA->SetTargetActor(InTargetActor);
	SwitchOffConditionB->SetTargetActor(InTargetActor);
}

void UMotionWarpingSwitchOffCompositeCondition::SetMotionWarpingTarget(const FMotionWarpingTarget* InMotionWarpingTarget)
{
	Super::SetMotionWarpingTarget(InMotionWarpingTarget);
	
	SwitchOffConditionA->SetMotionWarpingTarget(InMotionWarpingTarget);
	SwitchOffConditionB->SetMotionWarpingTarget(InMotionWarpingTarget);
}

void UMotionWarpingSwitchOffCompositeCondition::SetWarpTargetForDestination(const FMotionWarpingTarget* InMotionWarpingTarget)
{
	Super::SetWarpTargetForDestination(InMotionWarpingTarget);

	if (ensureMsgf(IsValid(SwitchOffConditionA), TEXT("Switch off condition A not setup in composite switch off condition on actor %s"), *OwnerActor->GetName())
		&& ensureMsgf(IsValid(SwitchOffConditionB), TEXT("Switch off condition B not setup in composite switch off condition on actor %s"), *OwnerActor->GetName()))
	{
		SwitchOffConditionA->SetWarpTargetForDestination(InMotionWarpingTarget);
		SwitchOffConditionB->SetWarpTargetForDestination(InMotionWarpingTarget);
	}
}

bool UMotionWarpingSwitchOffCompositeCondition::OnCheck() const
{
	if (ensureMsgf(IsValid(SwitchOffConditionA), TEXT("Switch off condition A not setup in composite switch off condition on actor %s"), *OwnerActor->GetName())
		&& ensureMsgf(IsValid(SwitchOffConditionB), TEXT("Switch off condition B not setup in composite switch off condition on actor %s"), *OwnerActor->GetName()))
	{
		if (LogicOperator == ESwitchOffConditionCompositeOp::Or)
		{
			return SwitchOffConditionA->Check() || SwitchOffConditionB->Check();
		}
		return SwitchOffConditionA->Check() && SwitchOffConditionB->Check();
	}

	return false;
}

FString UMotionWarpingSwitchOffCompositeCondition::ExtraDebugInfo() const
{
	return FString::Printf(TEXT("%s %s %s"),
		*SwitchOffConditionA->ExtraDebugInfo(),
		*(LogicOperator == ESwitchOffConditionCompositeOp::Or ? FString("OR") : FString("AND")),
		*SwitchOffConditionB->ExtraDebugInfo());
}

bool UMotionWarpingSwitchOffCompositeCondition::IsConditionValid() const
{
	return IsValid(SwitchOffConditionA) && IsValid(SwitchOffConditionB)
		&& SwitchOffConditionA->IsConditionValid() && SwitchOffConditionB->IsConditionValid();
}

UMotionWarpingSwitchOffCompositeCondition* UMotionWarpingSwitchOffCompositeCondition::CreateSwitchOffCompositeCondition(AActor* InOwnerActor, ESwitchOffConditionEffect InEffect, UMotionWarpingSwitchOffCondition* InSwitchOffConditionA, ESwitchOffConditionCompositeOp InLogicOperator, UMotionWarpingSwitchOffCondition* InSwitchOffConditionB, bool bInUseWarpTargetAsTargetLocation, AActor* InTargetActor)
{
	UMotionWarpingSwitchOffCompositeCondition* SwitchOffCompositeCondition = NewObject<UMotionWarpingSwitchOffCompositeCondition>();
	SwitchOffCompositeCondition->OwnerActor = InOwnerActor;
	SwitchOffCompositeCondition->Effect = InEffect;
	SwitchOffCompositeCondition->SwitchOffConditionA = InSwitchOffConditionA;
	SwitchOffCompositeCondition->LogicOperator = InLogicOperator;
	SwitchOffCompositeCondition->SwitchOffConditionB = InSwitchOffConditionB;
	SwitchOffCompositeCondition->bUseWarpTargetAsTargetLocation = bInUseWarpTargetAsTargetLocation;
	SwitchOffCompositeCondition->TargetActor = InTargetActor;

	return SwitchOffCompositeCondition;
}

bool UMotionWarpingSwitchOffBlueprintableCondition::OnCheck() const
{
	return BP_Check(OwnerActor, TargetActor, GetTargetLocation(), bUseWarpTargetAsTargetLocation);
}

FString UMotionWarpingSwitchOffBlueprintableCondition::ExtraDebugInfo() const
{
	return BP_ExtraDebugInfo(OwnerActor, TargetActor, GetTargetLocation(), bUseWarpTargetAsTargetLocation);
}

UMotionWarpingSwitchOffBlueprintableCondition* UMotionWarpingSwitchOffBlueprintableCondition::CreateSwitchOffBlueprintableCondition(AActor* InOwnerActor, ESwitchOffConditionEffect InEffect, TSubclassOf<UMotionWarpingSwitchOffBlueprintableCondition> InBlueprintableCondition, bool bInUseWarpTargetAsTargetLocation, AActor* InTargetActor)
{
	UMotionWarpingSwitchOffBlueprintableCondition* SwitchOffBlueprintableCondition = NewObject<UMotionWarpingSwitchOffBlueprintableCondition>(GetTransientPackage(), InBlueprintableCondition);
	SwitchOffBlueprintableCondition->OwnerActor = InOwnerActor;
	SwitchOffBlueprintableCondition->Effect = InEffect;
	SwitchOffBlueprintableCondition->bUseWarpTargetAsTargetLocation = bInUseWarpTargetAsTargetLocation;
	SwitchOffBlueprintableCondition->TargetActor = InTargetActor;

	return SwitchOffBlueprintableCondition;
}

UWorld* UMotionWarpingSwitchOffBlueprintableCondition::GetWorld() const
{
	if (IsValid(OwnerActor))
	{
		return OwnerActor->GetWorld();
	}

	return nullptr;
}

FString UMotionWarpingSwitchOffBlueprintableCondition::BP_ExtraDebugInfo_Implementation(const AActor* InOwnerActor, const AActor* InTargetActor, FVector InTargetLocation, bool bInUseWarpTargetAsTargetLocation) const
{
	return FString("No extra debug info. Override BP_ExtraDebugInfo to add it.");
}

bool UMotionWarpingSwitchOffBlueprintableCondition::BP_Check_Implementation(const AActor* InOwnerActor, const AActor* InTargetActor, FVector InTargetLocation, bool bInUseWarpTargetAsTargetLocation) const
{
	return false;
}
