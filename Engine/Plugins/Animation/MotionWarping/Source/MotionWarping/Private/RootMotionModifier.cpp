// Copyright Epic Games, Inc. All Rights Reserved.

#include "RootMotionModifier.h"
#include "GameFramework/Character.h"
#include "Components/CapsuleComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/World.h"
#include "MotionWarpingComponent.h"
#include "MotionWarpingAdapter.h"
#include "DrawDebugHelpers.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RootMotionModifier)


namespace FMotionWarpingRuntimeCVars
{
	// @todo: Switch this to default true after more testing.
	bool bUseMeshScale = false;
	static FAutoConsoleVariableRef CVarMotionWarping(
		TEXT("a.MotionWarping.UseMeshScale"),
		bUseMeshScale,
		TEXT("Account for mesh scale when calculating warp points (see CachedOffsetFromWarpPoint)."));
}

// FMotionWarpingTarget
///////////////////////////////////////////////////////////////

FMotionWarpingTarget::FMotionWarpingTarget(const FName& InName, const USceneComponent* InComp, FName InBoneName, bool bInFollowComponent, EWarpTargetLocationOffsetDirection InLocationOffsetDirection, const AActor* InAvatarActor, const FVector& InLocOffset, const FRotator& InRotOffset)
{
	if (ensure(InComp))
	{
		Name = InName;
		Component = InComp;
		BoneName = InBoneName;
		bFollowComponent = bInFollowComponent;
		LocationOffset = InLocOffset;
		RotationOffset = InRotOffset;
		AvatarActor = InAvatarActor;
		LocationOffsetDirection = InLocationOffsetDirection;

		FTransform Transform = FTransform::Identity;
		if (BoneName != NAME_None)
		{
			Transform = FMotionWarpingTarget::GetTargetTransformFromComponent(InComp, InBoneName);
		}
		else
		{
			Transform = InComp->GetComponentTransform();
		}

		CacheOffset(Transform);
		RecalculateOffset(Transform);

		Location = Transform.GetLocation();
		Rotation = Transform.Rotator();
	}
}

FMotionWarpingTarget::FMotionWarpingTarget(const FName& InName, const USceneComponent* InComp, FName InBoneName, bool bInFollowComponent, const FVector& InLocOffset, const FRotator& InRotOffset)
	: FMotionWarpingTarget(InName, InComp, InBoneName, bInFollowComponent, EWarpTargetLocationOffsetDirection::TargetsForwardVector, nullptr, InLocOffset, InRotOffset)
{
}

FTransform FMotionWarpingTarget::GetTargetTransformFromComponent(const USceneComponent* Comp, const FName& BoneName)
{
	if (Comp == nullptr)
	{
		UE_LOGF(LogMotionWarping, Warning, "FMotionWarpingTarget::GetTargetTransformFromComponent: Invalid Component");
		return FTransform::Identity;
	}

	if (Comp->DoesSocketExist(BoneName) == false)
	{
		UE_LOGF(LogMotionWarping, Warning, "FMotionWarpingTarget::GetTargetTransformFromComponent: Invalid Bone or Socket. Comp: %ls Owner: %ls BoneName: %ls",
			*GetNameSafe(Comp), *GetNameSafe(Comp->GetOwner()), *BoneName.ToString());

		return Comp->GetComponentTransform();
	}

	return Comp->GetSocketTransform(BoneName);
}

void URootMotionModifier_Warp::Serialize(FArchive& Ar)
{
	// Handle change of default blend type
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.IsLoading())
	{
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			// Important: this is done before loading so if data has changed from default it still works
			AddTranslationEasingFunc = EAlphaBlendOption::Linear;
		}
	}

	// Now load so any other deprecation fixup can work properly
	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (CustomVersion < FFortniteMainBranchObjectVersion::RootMotionModifierRotationOffset)
		{
			if (RotationType == EMotionWarpRotationType::OppositeDefault)
			{
				RotationType = EMotionWarpRotationType::Default;
				AdditionalRotationOffset.Yaw += 180.0f;
				AdditionalRotationOffset.Normalize();
			}
			else if ( RotationType == EMotionWarpRotationType::OppositeFacing )
			{
				RotationType = EMotionWarpRotationType::Facing;
				AdditionalRotationOffset.Yaw += 180.0f;
				AdditionalRotationOffset.Normalize();
			}
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

FTransform FMotionWarpingTarget::GetTargetTrasform() const
{
	if (Component.IsValid() && bFollowComponent)
	{
		FTransform Transform = FTransform::Identity;
		if (BoneName != NAME_None)
		{
			Transform = FMotionWarpingTarget::GetTargetTransformFromComponent(Component.Get(), BoneName);
		}
		else
		{
			Transform = Component->GetComponentTransform();
		}

		RecalculateOffset(Transform);
		return FTransform(Transform.GetRotation(), Transform.GetLocation());
	}

	return FTransform(Rotation, Location);
}

/*	Because vector from target to owner changes during warping, offset needs to be cached. */
void FMotionWarpingTarget::CacheOffset(const FTransform& InTransform)
{
	// Forward offset doesn't need to be cached if it's the only one used. Otherwise, cache it too.
	bCacheForwardOffset =
		LocationOffsetDirection == EWarpTargetLocationOffsetDirection::VectorFromTargetToOwner &&
			LocationOffset.X > SMALL_NUMBER &&
				(LocationOffset.Y > SMALL_NUMBER || LocationOffset.Z > SMALL_NUMBER);
	
	if (LocationOffsetDirection == EWarpTargetLocationOffsetDirection::VectorFromTargetToOwner)
	{
		if (AvatarActor.IsValid())
		{
			const FVector ContextVector = (AvatarActor->GetActorLocation() - InTransform.GetLocation()).GetSafeNormal();
			const FVector RightVector = -FVector::CrossProduct(ContextVector, FVector::UpVector);

			if (bCacheForwardOffset)
			{
				CachedForwardOffset = (ContextVector * LocationOffset.X);
			}

			CachedRightOffset = RightVector * LocationOffset.Y;
			CachedUpOffset = -FVector::CrossProduct(RightVector, ContextVector) * LocationOffset.Z;
		}
		else
		{
			UE_LOGF(LogMotionWarping, Warning, "Motion warping offset is set to VectorFromTargetToOwner but avator actor is invalid")
		}
	}
}

void FMotionWarpingTarget::RecalculateOffset(FTransform& InTransform) const
{
	FVector Offset = FVector::ZeroVector;
	
	switch (LocationOffsetDirection)
	{
		case EWarpTargetLocationOffsetDirection::TargetsForwardVector:
			Offset = LocationOffset;
			break;
			
		case EWarpTargetLocationOffsetDirection::VectorFromTargetToOwner:
			if (AvatarActor.IsValid())
			{
				if (bCacheForwardOffset)
				{
					Offset = Component->GetComponentTransform().Inverse().TransformVector(CachedForwardOffset + CachedUpOffset + CachedRightOffset);
				}
				else
				{
					const FVector ContextVector = (AvatarActor->GetActorLocation() - InTransform.GetLocation()).GetSafeNormal();
					const FVector ForwardOffset = (ContextVector * LocationOffset.X);
					
					Offset = Component->GetComponentTransform().Inverse().TransformVector(ForwardOffset + CachedUpOffset + CachedRightOffset);
				}
			}
			else
			{
				UE_LOGF(LogMotionWarping, Warning, "Motion warping offset is set to VectorFromOwnerToTarget but avator actor is invalid")
			}
			break;
			
		case EWarpTargetLocationOffsetDirection::WorldSpace:
			Offset = Component->GetComponentTransform().Inverse().TransformVector(LocationOffset);
			break;
			
		default:
			checkNoEntry();
	}
	
	InTransform = FTransform(RotationOffset, Offset) * InTransform;
}

// URootMotionModifier
///////////////////////////////////////////////////////////////

URootMotionModifier::URootMotionModifier(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UMotionWarpingComponent* URootMotionModifier::GetOwnerComponent() const
{
	return Cast<UMotionWarpingComponent>(GetOuter());
}

UMotionWarpingBaseAdapter* URootMotionModifier::GetOwnerAdapter() const
{
	UMotionWarpingComponent* OwnerComp = GetOwnerComponent();
	return OwnerComp ? OwnerComp->GetOwnerAdapter() : nullptr;
}

AActor* URootMotionModifier::GetActorOwner() const
{
	if (UMotionWarpingBaseAdapter* OwnerAdapter = GetOwnerAdapter())
	{
		return OwnerAdapter->GetActor();
	}

	return nullptr;
}

TSharedPtr<FRootMotionModifierSwapState> URootMotionModifier::GetSwapState() const
{
	TSharedPtr<FRootMotionModifierSwapState> Result = MakeShared<FRootMotionModifierSwapState>();
	PopulateSwapState(Result);
	return Result;
}

void URootMotionModifier::PopulateSwapState(TSharedPtr<FRootMotionModifierSwapState> InSwapState) const
{
	InSwapState->StartTime = StartTime;
	InSwapState->EndTime = EndTime;
	InSwapState->PreviousPosition = PreviousPosition;
	InSwapState->CurrentPosition = CurrentPosition;
	InSwapState->Weight = Weight;
	InSwapState->PlayRate = PlayRate;
	InSwapState->StartTransform = StartTransform;
	InSwapState->ActualStartTime = ActualStartTime;
	InSwapState->TotalRootMotionWithinWindow = TotalRootMotionWithinWindow;
	InSwapState->State = State;
}

void URootMotionModifier::RestoreFromSwapState(TSharedPtr<FRootMotionModifierSwapState> InSwapState)
{
	StartTime = InSwapState->StartTime;
	EndTime = InSwapState->EndTime;
	PreviousPosition = InSwapState->PreviousPosition;
	CurrentPosition = InSwapState->CurrentPosition;
	Weight = InSwapState->Weight;
	PlayRate = InSwapState->PlayRate;
	StartTransform = InSwapState->StartTransform;
	ActualStartTime = InSwapState->ActualStartTime;
	TotalRootMotionWithinWindow = InSwapState->TotalRootMotionWithinWindow;
	State = InSwapState->State;
}

void URootMotionModifier::Update(const FMotionWarpingUpdateContext& Context)
{
	const AActor* ActorOwner = GetActorOwner();

	// Mark for removal if our animation is not relevant anymore
	if (!Context.Animation.IsValid() || Context.Animation.Get() != Animation)
	{
		UE_LOGF(LogMotionWarping, Verbose, "MotionWarping: Marking RootMotionModifier for removal. Reason: Animation is not valid. Char: %ls Current Animation: %ls. Window: Animation: %ls [%f %f] [%f %f]",
			*GetNameSafe(ActorOwner), *GetNameSafe(Context.Animation.Get()), *GetNameSafe(Animation.Get()), StartTime, EndTime, PreviousPosition, CurrentPosition);

		SetState(ERootMotionModifierState::MarkedForRemoval);
		return;
	}

	// Update playback times and weight
	PreviousPosition = Context.PreviousPosition;
	CurrentPosition = Context.CurrentPosition;
	Weight = Context.Weight;
	PlayRate = Context.PlayRate;

	// Mark for removal if the animation already passed the warping window
	if (PreviousPosition >= EndTime)
	{
		UE_LOGF(LogMotionWarping, Verbose, "MotionWarping: Marking RootMotionModifier for removal. Reason: Window has ended. Char: %ls Animation: %ls [%f %f] [%f %f]",
			*GetNameSafe(ActorOwner), *GetNameSafe(Animation.Get()), StartTime, EndTime, PreviousPosition, CurrentPosition);

		SetState(ERootMotionModifierState::MarkedForRemoval);
		return;
	}

	// Mark for removal if we jumped to a position outside the warping window
	if (State == ERootMotionModifierState::Active && PreviousPosition < EndTime && (CurrentPosition > EndTime || CurrentPosition < StartTime))
	{
		const float ExpectedDelta = Context.DeltaSeconds * Context.PlayRate;
		const float ActualDelta = CurrentPosition - PreviousPosition;
		if (!FMath::IsNearlyZero(FMath::Abs(ActualDelta - ExpectedDelta), KINDA_SMALL_NUMBER))
		{
			UE_LOGF(LogMotionWarping, Verbose, "MotionWarping: Marking RootMotionModifier for removal. Reason: CurrentPosition manually changed. PrevPos: %f CurrPos: %f DeltaTime: %f ExpectedDelta: %f ActualDelta: %f",
				PreviousPosition, CurrentPosition, Context.DeltaSeconds, ExpectedDelta, ActualDelta);

			SetState(ERootMotionModifierState::MarkedForRemoval);
			return;
		}
	}

	// Check if we are inside the warping window
	if (PreviousPosition >= StartTime && PreviousPosition < EndTime)
	{
		// If we were waiting, switch to active
		if (GetState() == ERootMotionModifierState::Waiting)
		{
			SetState(ERootMotionModifierState::Active);
		}
	}

	if (State == ERootMotionModifierState::Active)
	{
		if (UMotionWarpingComponent* OwnerComp = GetOwnerComponent())
		{
			OnUpdateDelegate.ExecuteIfBound(OwnerComp, this);
		}
	}
}

void URootMotionModifier::SetState(ERootMotionModifierState NewState)
{
	if (State != NewState)
	{
		ERootMotionModifierState LastState = State;

		State = NewState;

		OnStateChanged(LastState);
	}
}

void URootMotionModifier::OnStateChanged(ERootMotionModifierState LastState)
{
	if (UMotionWarpingComponent* OwnerComp = GetOwnerComponent())
	{
		if (LastState != ERootMotionModifierState::Active && State == ERootMotionModifierState::Active)
		{
			const UMotionWarpingBaseAdapter* OwnerAdapter = GetOwnerAdapter();

			checkf(OwnerAdapter, TEXT("Root motion modifiers expect an owner and adapter"));

			const FVector CurrentLocation = OwnerAdapter->GetVisualRootLocation();
			const FQuat CurrentRotation = OwnerAdapter->GetCurrentRotation();
			
			ActualStartTime = PreviousPosition;

			StartTransform = FTransform(CurrentRotation, CurrentLocation);

			TotalRootMotionWithinWindow = UMotionWarpingUtilities::ExtractRootMotionFromAnimation(Animation.Get(), StartTime, EndTime);

			OnActivateDelegate.ExecuteIfBound(OwnerComp, this);
		}
		else if (LastState == ERootMotionModifierState::Active && (State == ERootMotionModifierState::Disabled || State == ERootMotionModifierState::MarkedForRemoval))
		{
			OnDeactivateDelegate.ExecuteIfBound(OwnerComp, this);
		}
	}
}

URootMotionModifier_Warp::URootMotionModifier_Warp(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void URootMotionModifier_Warp::Update(const FMotionWarpingUpdateContext& Context)
{
	// Update playback times and state
	Super::Update(Context);

	// Cache sync point transform and trigger OnTargetTransformChanged if needed
	const UMotionWarpingComponent* OwnerComp = GetOwnerComponent();
	if (OwnerComp && GetState() == ERootMotionModifierState::Active)
	{
		const FMotionWarpingTarget* WarpTargetPtr = OwnerComp->FindWarpTarget(WarpTargetName);

		// Disable if there is no target for us
		if (WarpTargetPtr == nullptr)
		{
			UE_LOGF(LogMotionWarping, Verbose, "MotionWarping: Marking RootMotionModifier as Disabled. Reason: Invalid Warp Target (%ls). Char: %ls Animation: %ls [%f %f] [%f %f]",
				*WarpTargetName.ToString(), *GetNameSafe(OwnerComp->GetOwner()), *GetNameSafe(Animation.Get()), StartTime, EndTime, PreviousPosition, CurrentPosition);

			SetState(ERootMotionModifierState::Disabled);
			return;
		}

		bRootMotionPaused = WarpTargetPtr->bRootMotionPaused;
		bWarpingPaused = WarpTargetPtr->bWarpingPaused;
		
		// Get the warp point sent by the game
		FTransform WarpPointTransformGame = WarpTargetPtr->GetTargetTrasform();

		// Cache the rotation offset to later apply when rotation warping 
		RotationOffset = WarpTargetPtr->RotationOffset.Quaternion();
		
		// Initialize our target transform (where the root should end at the end of the window) with the warp point sent by the game
		FTransform TargetTransform = WarpPointTransformGame;

		// Check if a warp point is defined in the animation. If so, we need to extract it and offset the target transform 
		// the same amount the root bone is offset from the warp point in the animation
		if (WarpPointAnimProvider != EWarpPointAnimProvider::None)
		{
			if (!CachedOffsetFromWarpPoint.IsSet())
			{
				if (const UMotionWarpingBaseAdapter* OwnerAdapter = GetOwnerAdapter())
				{
					if (WarpPointAnimProvider == EWarpPointAnimProvider::Static)
					{
						CachedOffsetFromWarpPoint = UMotionWarpingUtilities::CalculateRootTransformRelativeToWarpPointAtTime(*OwnerAdapter, GetAnimation(), EndTime, WarpPointAnimTransform);
					}
					else if (WarpPointAnimProvider == EWarpPointAnimProvider::Bone)
					{
						CachedOffsetFromWarpPoint = UMotionWarpingUtilities::CalculateRootTransformRelativeToWarpPointAtTime(*OwnerAdapter, GetAnimation(), EndTime, WarpPointAnimBoneName);
					}

					// Scale the animation-space offset by mesh world scale so the target accounts for the scaled skeleton
					if (FMotionWarpingRuntimeCVars::bUseMeshScale && CachedOffsetFromWarpPoint.IsSet())
					{
						if (const USkeletalMeshComponent* Mesh = OwnerAdapter->GetMesh())
						{
							CachedOffsetFromWarpPoint.GetValue().ScaleTranslation(Mesh->GetComponentTransform().GetScale3D());
						}
					}
				}
			}

			// Update Target Transform based on the offset between the root and the warp point in the animation
			TargetTransform = CachedOffsetFromWarpPoint.GetValue() * WarpPointTransformGame;
		}
		
		CachedTargetTransform *= RootMotionRemainingAfterNotify.Inverse();
		
		if (!CachedTargetTransform.Equals(TargetTransform))
		{
			CachedTargetTransform = TargetTransform;

			OnTargetTransformChanged();
		}
	}
}

void URootMotionModifier_Warp::OnTargetTransformChanged()
{
 	if (const UMotionWarpingBaseAdapter* WarpingAdapter = GetOwnerAdapter())
	{
		ActualStartTime = PreviousPosition;

		const FQuat CurrentRotation = WarpingAdapter->GetCurrentRotation();
		const FVector CurrentLocation = WarpingAdapter->GetVisualRootLocation();
		StartTransform = FTransform(CurrentRotation, CurrentLocation);
	}
}

void URootMotionModifier_Warp::OnStateChanged(ERootMotionModifierState LastState)
{
	Super::OnStateChanged(LastState);

	if (bSubtractRemainingRootMotion)
	{
		RootMotionRemainingAfterNotify = UMotionWarpingUtilities::ExtractRootMotionFromAnimation(Animation.Get(), EndTime, Animation.Get()->GetPlayLength());
	}
}


TSharedPtr<FRootMotionModifierSwapState> URootMotionModifier_Warp::GetSwapState() const
{
	TSharedPtr<FRootMotionModifierSwapState> Result = MakeShared<FRootMotionModifier_WarpSwapState>();
	PopulateSwapState(Result);
	return Result;
}

void URootMotionModifier_Warp::PopulateSwapState(TSharedPtr<FRootMotionModifierSwapState> InSwapState) const
{
	Super::PopulateSwapState(InSwapState);

	TSharedPtr<FRootMotionModifier_WarpSwapState> WarpSwapState = StaticCastSharedPtr<FRootMotionModifier_WarpSwapState>(InSwapState);

	WarpSwapState->AdditionalRotationOffset = AdditionalRotationOffset;
	WarpSwapState->CachedTargetTransform = CachedTargetTransform;
	WarpSwapState->RootMotionRemainingAfterNotify = RootMotionRemainingAfterNotify;
	WarpSwapState->CachedOffsetFromWarpPoint = CachedOffsetFromWarpPoint;
	WarpSwapState->RotationOffset = RotationOffset;
	WarpSwapState->bWarpingPaused = bWarpingPaused;
	WarpSwapState->bRootMotionPaused = bRootMotionPaused;
}

void URootMotionModifier_Warp::RestoreFromSwapState(TSharedPtr<FRootMotionModifierSwapState> InSwapState)
{
	Super::RestoreFromSwapState(InSwapState);

	TSharedPtr<FRootMotionModifier_WarpSwapState> WarpSwapState = StaticCastSharedPtr<FRootMotionModifier_WarpSwapState>(InSwapState);

	AdditionalRotationOffset = WarpSwapState->AdditionalRotationOffset;
	CachedTargetTransform = WarpSwapState->CachedTargetTransform;
	RootMotionRemainingAfterNotify = WarpSwapState->RootMotionRemainingAfterNotify;
	CachedOffsetFromWarpPoint = WarpSwapState->CachedOffsetFromWarpPoint;
	RotationOffset = WarpSwapState->RotationOffset;
	bWarpingPaused = WarpSwapState->bWarpingPaused;
	bRootMotionPaused = WarpSwapState->bRootMotionPaused;
}

FQuat URootMotionModifier_Warp::GetTargetRotation() const
{
	switch (RotationType)
	{
		case EMotionWarpRotationType::Default:
			return FQuat::MakeFromRotator(AdditionalRotationOffset) * CachedTargetTransform.GetRotation();
		
		case EMotionWarpRotationType::Facing:
			if (const AActor* ActorOwner = GetActorOwner())
			{
				const FTransform& ActorTransform = ActorOwner->GetActorTransform();
				const FVector ToSyncPoint = (CachedTargetTransform.GetLocation() - ActorTransform.GetLocation()).GetSafeNormal2D();
				return FQuat::MakeFromRotator(AdditionalRotationOffset) * FRotationMatrix::MakeFromXZ(ToSyncPoint, FVector::UpVector).ToQuat();
			}
			break;
		
		default:
			checkNoEntry();
	}

	return FQuat::Identity;
}

FQuat URootMotionModifier_Warp::WarpRotation(const FTransform& RootMotionDelta, const FTransform& RootMotionTotal, float DeltaSeconds)
{
	if (bRootMotionPaused)
	{
		return FQuat::Identity;
	}

	if (bWarpingPaused)
	{
		return RootMotionDelta.GetRotation();
	}
	
	FQuat CurrentRotation;
	FQuat TargetRotation;

	if (const UMotionWarpingBaseAdapter* WarpingAdapter = GetOwnerAdapter())
	{
		CurrentRotation = WarpingAdapter->GetCurrentRotation() * WarpingAdapter->GetBaseVisualRotationOffset();
		TargetRotation = CurrentRotation.Inverse() * (GetTargetRotation() * WarpingAdapter->GetBaseVisualRotationOffset() * RootMotionRemainingAfterNotify.GetRotation().Inverse());
	}
	else
	{
		// No owner, no warping possible
		return FQuat::Identity;
	}

	const FQuat TotalRootMotionRotation = RootMotionTotal.GetRotation();

	if (RotationMethod == EMotionWarpRotationMethod::Scale)
	{
		FRotator TotalRotator(TotalRootMotionRotation);
		FRotator TargetRotator(TargetRotation);
		const double YawDiff = FMath::FindDeltaAngleDegrees(TotalRotator.Yaw, TargetRotator.Yaw);
		const double PitchDiff = FMath::FindDeltaAngleDegrees(TotalRotator.Pitch, TargetRotator.Pitch);
		// To properly compute scale factor target rotation needs to be relative to total rotation.
		// To avoid cases like 170 & -170 resulting in -1 scale factor rather than 1.11.
		const double YawScale = FMath::IsNearlyZero(TotalRotator.Yaw) ? 0.0 : (TotalRotator.Yaw + YawDiff) / TotalRotator.Yaw;
		const double PitchScale = FMath::IsNearlyZero(TotalRotator.Pitch) ? 0.0 : (TotalRotator.Pitch + PitchDiff) / TotalRotator.Pitch;
		const float MaxRotation = WarpMaxRotationRate * DeltaSeconds;
		FRotator ScaledDeltaRotation(RootMotionDelta.GetRotation());
		ScaledDeltaRotation.Yaw = FMath::Clamp(ScaledDeltaRotation.Yaw * YawScale, -MaxRotation, MaxRotation);
		ScaledDeltaRotation.Pitch = FMath::Clamp(ScaledDeltaRotation.Pitch * PitchScale, -MaxRotation, MaxRotation);
		return ScaledDeltaRotation.Quaternion();
	}

	const float TimeRemaining = (EndTime - PreviousPosition) * WarpRotationTimeMultiplier;
	const float PlayRateAdjustedDeltaSeconds = DeltaSeconds * PlayRate;
	const float Alpha = FMath::Clamp(PlayRateAdjustedDeltaSeconds / TimeRemaining, 0.f, 1.f);
	FQuat TargetRotThisFrame = FQuat::Slerp(TotalRootMotionRotation, TargetRotation, Alpha);

	if (RotationMethod != EMotionWarpRotationMethod::Slerp)
	{
		const float AngleDeltaThisFrame = TotalRootMotionRotation.AngularDistance(TargetRotThisFrame);
		const float MaxAngleDelta = FMath::Abs(FMath::DegreesToRadians(PlayRateAdjustedDeltaSeconds * WarpMaxRotationRate));
		const float TotalAngleDelta = TotalRootMotionRotation.AngularDistance(TargetRotation);
		if (RotationMethod == EMotionWarpRotationMethod::ConstantRate && (TotalAngleDelta <= MaxAngleDelta))
		{
			TargetRotThisFrame = TargetRotation;
		}
		else if ((AngleDeltaThisFrame > MaxAngleDelta) || RotationMethod == EMotionWarpRotationMethod::ConstantRate)
		{
			const FVector CrossProduct = FVector::CrossProduct(TotalRootMotionRotation.Vector(), TargetRotation.Vector());
			const float SignDirection = FMath::Sign(CrossProduct.Z);
			const FQuat ClampedRotationThisFrame = FQuat(FVector(0, 0, 1), MaxAngleDelta * SignDirection);
			TargetRotThisFrame = ClampedRotationThisFrame;
		}
	}

	FQuat DeltaOut = TargetRotThisFrame * TotalRootMotionRotation.Inverse();
	DeltaOut = DeltaOut.Equals(FQuat::Identity) ? FQuat::Identity : DeltaOut;
	
	return (DeltaOut * RootMotionDelta.GetRotation());
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
void URootMotionModifier_Warp::PrintLog(const FString& Name, const FTransform& OriginalRootMotion, const FTransform& WarpedRootMotion) const
{
	const AActor* ActorOwner = nullptr;

	FVector CurrentLocation;
	USkeletalMeshComponent* SkelMesh = nullptr;

	if (const UMotionWarpingBaseAdapter* WarpingAdapter = GetOwnerAdapter())
	{
		ActorOwner = WarpingAdapter->GetActor();
		SkelMesh = WarpingAdapter->GetMesh();
		CurrentLocation = WarpingAdapter->GetVisualRootLocation();
	}

	if (ActorOwner && SkelMesh)
	{
		const FVector CurrentToTarget = (GetTargetLocation() - CurrentLocation).GetSafeNormal2D();
		const FVector FutureLocation = CurrentLocation + (SkelMesh->ConvertLocalRootMotionToWorld(WarpedRootMotion)).GetTranslation();
		const FRotator CurrentRotation = ActorOwner->GetActorRotation();
		const FRotator FutureRotation = (WarpedRootMotion.GetRotation() * ActorOwner->GetActorQuat()).Rotator();
		const float Dot = FVector::DotProduct(ActorOwner->GetActorForwardVector(), CurrentToTarget);
		const float CurrentDist2D = FVector::Dist2D(GetTargetLocation(), CurrentLocation);
		const float FutureDist2D = FVector::Dist2D(GetTargetLocation(), FutureLocation);
		const float DeltaSeconds = ActorOwner->GetWorld()->GetDeltaSeconds();
		const float Speed = WarpedRootMotion.GetTranslation().Size() / DeltaSeconds;
		const float EndTimeOffset = CurrentPosition - EndTime;

		UE_LOGF(LogMotionWarping, Log, "%ls NetMode: %d Char: %ls Anim: %ls Win: [%f %f][%f %f] DT: %f WT: %f ETOffset: %f Dist2D: %f Z: %f FDist2D: %f FZ: %f Dot: %f Delta: %ls (%f) FDelta: %ls (%f) Speed: %f Loc: %ls FLoc: %ls Rot: %ls FRot: %ls",
			*Name, (int32)ActorOwner->GetWorld()->GetNetMode(), *GetNameSafe(ActorOwner), *GetNameSafe(Animation.Get()), StartTime, EndTime, PreviousPosition, CurrentPosition, DeltaSeconds, ActorOwner->GetWorld()->GetTimeSeconds(), EndTimeOffset,
			CurrentDist2D, (GetTargetLocation().Z - CurrentLocation.Z), FutureDist2D, (GetTargetLocation().Z - FutureLocation.Z), Dot,
			*OriginalRootMotion.GetTranslation().ToString(), OriginalRootMotion.GetTranslation().Size(), *WarpedRootMotion.GetTranslation().ToString(), WarpedRootMotion.GetTranslation().Size(), Speed,
			*CurrentLocation.ToString(), *FutureLocation.ToString(), *CurrentRotation.ToCompactString(), *FutureRotation.ToCompactString());
	}
}
#endif

// URootMotionModifier_SimpleWarp
///////////////////////////////////////////////////////////////

UDEPRECATED_RootMotionModifier_SimpleWarp::UDEPRECATED_RootMotionModifier_SimpleWarp(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FTransform UDEPRECATED_RootMotionModifier_SimpleWarp::ProcessRootMotion(const FTransform& InRootMotion, float DeltaSeconds)
{
	const ACharacter* CharacterOwner = nullptr;

	if (const UMotionWarpingBaseAdapter* Adapter = GetOwnerAdapter())
	{
		CharacterOwner = Cast<ACharacter>(Adapter->GetActor());
	}

	if (CharacterOwner == nullptr)
	{
		return InRootMotion;
	}

	const FTransform& CharacterTransform = CharacterOwner->GetActorTransform();

	FTransform FinalRootMotion = InRootMotion;

	const FTransform RootMotionTotal = UMotionWarpingUtilities::ExtractRootMotionFromAnimation(Animation.Get(), PreviousPosition, EndTime);

	if (bWarpTranslation)
	{
		FVector DeltaTranslation = InRootMotion.GetTranslation();

		const FTransform RootMotionDelta = UMotionWarpingUtilities::ExtractRootMotionFromAnimation(Animation.Get(), PreviousPosition, FMath::Min(CurrentPosition, EndTime));

		const float HorizontalDelta = RootMotionDelta.GetTranslation().Size2D();
		const float HorizontalTarget = FVector::Dist2D(CharacterTransform.GetLocation(), GetTargetLocation());
		const float HorizontalOriginal = RootMotionTotal.GetTranslation().Size2D();
		const float HorizontalTranslationWarped = !FMath::IsNearlyZero(HorizontalOriginal) ? ((HorizontalDelta * HorizontalTarget) / HorizontalOriginal) : 0.f;

		const FTransform MeshRelativeTransform = FTransform(CharacterOwner->GetBaseRotationOffset(), CharacterOwner->GetBaseTranslationOffset());
		const FTransform MeshTransform = MeshRelativeTransform * CharacterOwner->GetActorTransform();
		DeltaTranslation = MeshTransform.InverseTransformPositionNoScale(GetTargetLocation()).GetSafeNormal2D() * HorizontalTranslationWarped;

		if (!bIgnoreZAxis)
		{
			const float CapsuleHalfHeight = CharacterOwner->GetCapsuleComponent()->GetScaledCapsuleHalfHeight();
			const FVector CapsuleBottomLocation = (CharacterOwner->GetActorLocation() - FVector(0.f, 0.f, CapsuleHalfHeight));
			const float VerticalDelta = RootMotionDelta.GetTranslation().Z;
			const float VerticalTarget = GetTargetLocation().Z - CapsuleBottomLocation.Z;
			const float VerticalOriginal = RootMotionTotal.GetTranslation().Z;
			const float VerticalTranslationWarped = !FMath::IsNearlyZero(VerticalOriginal) ? ((VerticalDelta * VerticalTarget) / VerticalOriginal) : 0.f;

			DeltaTranslation.Z = VerticalTranslationWarped;
		}
		else
		{
			DeltaTranslation.Z = InRootMotion.GetTranslation().Z;
		}

		FinalRootMotion.SetTranslation(DeltaTranslation);
	}

	if (bWarpRotation)
	{
		const FQuat WarpedRotation = WarpRotation(InRootMotion, RootMotionTotal, DeltaSeconds);
		FinalRootMotion.SetRotation(WarpedRotation);
	}

	// Debug
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	const int32 DebugLevel = FMotionWarpingCVars::CVarMotionWarpingDebug.GetValueOnGameThread();
	if (DebugLevel == 1 || DebugLevel == 3)
	{
		PrintLog(TEXT("SimpleWarp"), InRootMotion, FinalRootMotion);
	}

	if (DebugLevel == 2 || DebugLevel == 3)
	{
		const float DrawDebugDuration = FMotionWarpingCVars::CVarMotionWarpingDrawDebugDuration.GetValueOnGameThread();
		DrawDebugCoordinateSystem(CharacterOwner->GetWorld(), GetTargetLocation(), GetTargetRotator(), 50.f, false, DrawDebugDuration, 0, 1.f);
	}
#endif

	return FinalRootMotion;
}

// URootMotionModifier_Scale
///////////////////////////////////////////////////////////////

URootMotionModifier_Scale* URootMotionModifier_Scale::AddRootMotionModifierScale(UMotionWarpingComponent* InMotionWarpingComp, const UAnimSequenceBase* InAnimation, float InStartTime, float InEndTime, FVector InScale)
{
	if (ensureAlways(InMotionWarpingComp))
	{
		URootMotionModifier_Scale* NewModifier = NewObject<URootMotionModifier_Scale>(InMotionWarpingComp);
		NewModifier->Animation = InAnimation;
		NewModifier->StartTime = InStartTime;
		NewModifier->EndTime = InEndTime;
		NewModifier->Scale = InScale;

		InMotionWarpingComp->AddModifier(NewModifier);

		return NewModifier;
	}

	return nullptr;
}


// URootMotionModifier_Blueprintable
///////////////////////////////////////////////////////////////

FTransform URootMotionModifier_Blueprintable::ProcessRootMotion(const FTransform& InRootMotion, float DeltaSeconds)
{
	check(IsInGameThread()); // checking since BP_ProcessRootMotion is not threadsafe
	if (const UMotionWarpingBaseAdapter* Adapter = GetOwnerAdapter())
	{
		return BP_ProcessRootMotion(Adapter->GetActor(), Adapter->GetMesh(), InRootMotion, DeltaSeconds);
	}

	return InRootMotion;
}
