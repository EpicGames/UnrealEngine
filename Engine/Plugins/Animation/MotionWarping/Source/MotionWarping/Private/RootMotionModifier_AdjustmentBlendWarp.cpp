// Copyright Epic Games, Inc. All Rights Reserved.

#include "RootMotionModifier_AdjustmentBlendWarp.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimInstance.h"
#include "AnimationUtils.h"
#include "BonePose.h"
#include "Components/SkeletalMeshComponent.h"
#include "DrawDebugHelpers.h"
#include "GameFramework/Character.h"
#include "MotionWarpingComponent.h"
#include "MotionWarpingAdapter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RootMotionModifier_AdjustmentBlendWarp)

DECLARE_CYCLE_STAT(TEXT("MotionWarping PrecomputeWarpedTracks"), STAT_MotionWarping_PrecomputeWarpedTracks, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("MotionWarping ExtractMotionDelta"), STAT_MotionWarping_ExtractMotionDelta, STATGROUP_Anim);

URootMotionModifier_AdjustmentBlendWarp::URootMotionModifier_AdjustmentBlendWarp(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void URootMotionModifier_AdjustmentBlendWarp::ExtractBoneTransformAtTime(FTransform& OutTransform, const FName& BoneName, float Time) const
{
	const int32 TrackIndex = Result.TrackNames.IndexOfByKey(BoneName);
	ExtractBoneTransformAtTime(OutTransform, TrackIndex, Time);
}

void URootMotionModifier_AdjustmentBlendWarp::ExtractBoneTransformAtTime(FTransform& OutTransform, int32 TrackIndex, float Time) const
{
	if (!Result.AnimationTracks.IsValidIndex(TrackIndex))
	{
		OutTransform = FTransform::Identity;
		return;
	}

	const int32 TotalFrames = FMath::Max(Result.AnimationTracks[TrackIndex].PosKeys.Num(), Result.AnimationTracks[TrackIndex].RotKeys.Num());
	const float TrackLength = (EndTime - ActualStartTime);
	const float TimePercent = (Time - ActualStartTime) / (EndTime - ActualStartTime);
	const float RemappedTime = TimePercent * TrackLength;
	FAnimationUtils::ExtractTransformFromTrack(Result.AnimationTracks[TrackIndex], static_cast<double>(RemappedTime), TotalFrames, TrackLength, EAnimInterpolationType::Linear, OutTransform);
}

void URootMotionModifier_AdjustmentBlendWarp::ExtractBoneTransformAtFrame(FTransform& OutTransform, int32 TrackIndex, int32 Frame) const
{
	if (!Result.AnimationTracks.IsValidIndex(TrackIndex) || !Result.AnimationTracks[TrackIndex].PosKeys.IsValidIndex(Frame))
	{
		OutTransform = FTransform::Identity;
		return;
	}

	FAnimationUtils::ExtractTransformForFrameFromTrack(Result.AnimationTracks[TrackIndex], Frame, OutTransform);
}

void URootMotionModifier_AdjustmentBlendWarp::OnTargetTransformChanged()
{
	const USkeletalMeshComponent* SkelMeshComp = nullptr;

	if (UMotionWarpingBaseAdapter* OwnerAdapter = GetOwnerAdapter())
	{
		SkelMeshComp = OwnerAdapter->GetMesh();
	}

	if (SkelMeshComp)
	{
		ActualStartTime = PreviousPosition;
		CachedRootMotion = UMotionWarpingUtilities::ExtractRootMotionFromAnimation(Animation.Get(), ActualStartTime, EndTime);
		CachedMeshRelativeTransform = SkelMeshComp->GetRelativeTransform();
		CachedMeshTransform = SkelMeshComp->GetComponentTransform();

		Result.AnimationTracks.Reset();
		Result.TrackNames.Reset();
	}
}

FTransform URootMotionModifier_AdjustmentBlendWarp::ProcessRootMotion(const FTransform& InRootMotion, float DeltaSeconds)
{
	// If warped tracks has not been generated yet, do it now
	if (Result.GetNum() == 0)
	{
		PrecomputeWarpedTracks();
	}

	// Extract root motion from warped tracks
	const FTransform FinalRootMotion = ExtractWarpedRootMotion();

	// Debug
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	const int32 DebugLevel = FMotionWarpingCVars::CVarMotionWarpingDebug.GetValueOnGameThread();
	if (DebugLevel == 1 || DebugLevel == 3)
	{
		PrintLog(TEXT("AdjustmentBlendWarp"), InRootMotion, FinalRootMotion);
	}

	if (DebugLevel == 2 || DebugLevel == 3)
	{
		const float DrawDebugDuration = FMotionWarpingCVars::CVarMotionWarpingDrawDebugDuration.GetValueOnGameThread();
		DrawDebugWarpedTracks(DrawDebugDuration);
	}
#endif

	return FinalRootMotion;
}

void URootMotionModifier_AdjustmentBlendWarp::PrecomputeWarpedTracks()
{
	SCOPE_CYCLE_COUNTER(STAT_MotionWarping_PrecomputeWarpedTracks);

	// First, extract pose at the end of the window for the bones we are going to warp

	const USkeletalMeshComponent* SkelMesh = nullptr;

	if (const UMotionWarpingBaseAdapter* OwnerAdapter = GetOwnerAdapter())
	{
		SkelMesh = OwnerAdapter->GetMesh();
	}

	if (!SkelMesh)
	{
		return;
	}

	const FBoneContainer& BoneContainer = SkelMesh->GetAnimInstance()->GetRequiredBones();

	// Init FBoneContainer with only the bones that we are interested in
	TArray<FBoneIndexType> RequiredBoneIndexArray;
	RequiredBoneIndexArray.Add(0);

	const bool bShouldWarpIKBones = bWarpIKBones && IKBones.Num() > 0;
	if (bShouldWarpIKBones)
	{
		for (const FName& BoneName : IKBones)
		{
			const int32 BoneIndex = BoneContainer.GetPoseBoneIndexForBoneName(BoneName);
			if (BoneIndex != INDEX_NONE)
			{
				RequiredBoneIndexArray.Add(BoneIndex);
			}
		}

		BoneContainer.GetReferenceSkeleton().EnsureParentsExistAndSort(RequiredBoneIndexArray);
	}

	// Init BoneContainer
	FBoneContainer RequiredBones(RequiredBoneIndexArray, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::DisallowAll), *BoneContainer.GetAsset());

	// Extract pose
	FCSPose<FCompactPose> CSPose;
	UMotionWarpingUtilities::ExtractComponentSpacePose(Animation.Get(), RequiredBones, EndTime, true, CSPose);

	// Second, calculate additive pose

	//Calculate additive translation for root bone
	FVector RootTargetLocation = CachedMeshTransform.InverseTransformPositionNoScale(GetTargetLocation());

	FVector RootTotalAdditiveTranslation = FVector::ZeroVector;
	if (bWarpTranslation)
	{
		RootTotalAdditiveTranslation = RootTargetLocation - CachedRootMotion.GetLocation();

		if (bIgnoreZAxis)
		{
			RootTotalAdditiveTranslation.Z = 0.f;
		}
	}

	// Calculate additive rotation for root bone
	FQuat RootTotalAdditiveRotation = FQuat::Identity;
	if (bWarpRotation)
	{
		const FQuat TargetRotation = GetTargetRotation();
		const FQuat OriginalRotation = CachedMeshRelativeTransform.GetRotation().Inverse() * (CachedRootMotion * CachedMeshTransform).GetRotation();
		RootTotalAdditiveRotation = FQuat::FindBetweenNormals(OriginalRotation.GetForwardVector(), TargetRotation.GetForwardVector());
	}

	// Init Additive Pose
	FCSPose<FCompactPose> AdditivePose;
	AdditivePose.InitPose(&RequiredBones);
	AdditivePose.SetComponentSpaceTransform(FCompactPoseBoneIndex(0), FTransform(RootTotalAdditiveRotation, RootTotalAdditiveTranslation));

	// Calculate and add additive pose for IK bones
	if (bShouldWarpIKBones)
	{
		const FTransform RootTargetPoseCS = FTransform((RootTotalAdditiveRotation * CachedRootMotion.GetRotation()), CachedRootMotion.GetTranslation() + RootTotalAdditiveTranslation);
		for (int32 Idx = 1; Idx < CSPose.GetPose().GetNumBones(); Idx++)
		{
			const FName BoneName = RequiredBones.GetReferenceSkeleton().GetBoneName(RequiredBones.GetBoneIndicesArray()[Idx]);
			if (IKBones.Contains(BoneName))
			{
				const int32 BoneIdx = Idx;
				const FTransform BonePoseCS = CSPose.GetComponentSpaceTransform(FCompactPoseBoneIndex(BoneIdx));

				const FTransform BoneTargetPoseCS = BonePoseCS * RootTargetPoseCS;
				const FTransform BoneOriginalPoseCS = BonePoseCS * CachedRootMotion;

				const FVector TotalAdditiveTranslation = BoneTargetPoseCS.GetLocation() - BoneOriginalPoseCS.GetLocation();

				AdditivePose.SetComponentSpaceTransform(FCompactPoseBoneIndex(Idx), FTransform(TotalAdditiveTranslation));
			}
		}
	}

	// Finally, run adjustment blending to generate the warped poses for each bone

	//@todo_fer: We could extract and cache this offline when the WarpingWindow is created
	const float SampleRate = 1 / 60.f;
	FMotionDeltaTrackContainer MotionDeltaTracks;
	URootMotionModifier_AdjustmentBlendWarp::ExtractMotionDeltaFromRange(RequiredBones, Animation.Get(), ActualStartTime, EndTime, SampleRate, MotionDeltaTracks);

	URootMotionModifier_AdjustmentBlendWarp::AdjustmentBlendWarp(RequiredBones, AdditivePose, MotionDeltaTracks, Result);
}

void URootMotionModifier_AdjustmentBlendWarp::ExtractMotionDeltaFromRange(const FBoneContainer& BoneContainer, const UAnimSequenceBase* Animation, float StartTime, float EndTime, float SampleRate, FMotionDeltaTrackContainer& OutMotionDeltaTracks)
{
	SCOPE_CYCLE_COUNTER(STAT_MotionWarping_ExtractMotionDelta);

	check(Animation);

	const int32 TotalFrames = FMath::CeilToInt((EndTime - StartTime) / SampleRate) + 1;
	const int32 TotalBones = BoneContainer.GetCompactPoseNumBones();

	FCompactPose FirstFramePose;
	UMotionWarpingUtilities::ExtractLocalSpacePose(Animation, BoneContainer, StartTime, false, FirstFramePose);
	const FTransform RootTransformFirstFrame = FirstFramePose[FCompactPoseBoneIndex(0)];

	FCSPose<FCompactPose> CSPose;
	UMotionWarpingUtilities::ExtractComponentSpacePose(Animation, BoneContainer, StartTime, false, CSPose);

	OutMotionDeltaTracks.Init(TotalBones);
	for (int32 BoneIdx = 0; BoneIdx < TotalBones; BoneIdx++)
	{
		const FTransform& BoneTransform = (CSPose.GetComponentSpaceTransform(FCompactPoseBoneIndex(BoneIdx)).GetRelativeTransform(RootTransformFirstFrame));

		FMotionDeltaTrack& NewTrack = OutMotionDeltaTracks.Tracks.AddDefaulted_GetRef();

		NewTrack.BoneTransformTrack.Reserve(TotalFrames);
		NewTrack.DeltaTranslationTrack.Reserve(TotalFrames);
		NewTrack.DeltaRotationTrack.Reserve(TotalFrames);

		NewTrack.BoneTransformTrack.Add(BoneTransform);
		NewTrack.DeltaTranslationTrack.Add(FVector::ZeroVector);
		NewTrack.DeltaRotationTrack.Add(FRotator::ZeroRotator);

		NewTrack.TotalTranslation = FVector::ZeroVector;
		NewTrack.TotalRotation = FRotator::ZeroRotator;
	}

	float Time = StartTime;
	while (Time < EndTime)
	{
		Time += FMath::Min(SampleRate, EndTime - Time);

		UMotionWarpingUtilities::ExtractComponentSpacePose(Animation, BoneContainer, Time, false, CSPose);

		for (int32 BoneIdx = 0; BoneIdx < TotalBones; BoneIdx++)
		{
			FMotionDeltaTrack& Track = OutMotionDeltaTracks.Tracks[BoneIdx];

			const FTransform& BoneTransform = (CSPose.GetComponentSpaceTransform(FCompactPoseBoneIndex(BoneIdx)).GetRelativeTransform(RootTransformFirstFrame));

			const FTransform& LastBoneTransform = Track.BoneTransformTrack.Last();

			Track.BoneTransformTrack.Add(BoneTransform);

			const FVector Translation = BoneTransform.GetTranslation();
			const FVector LastTranslation = LastBoneTransform.GetTranslation();
			const FVector DeltaTranslation(FMath::Abs(Translation.X - LastTranslation.X), FMath::Abs(Translation.Y - LastTranslation.Y), FMath::Abs(Translation.Z - LastTranslation.Z));
			Track.DeltaTranslationTrack.Add(DeltaTranslation);
			Track.TotalTranslation = Track.TotalTranslation + DeltaTranslation;

			const FRotator Rotation = BoneTransform.GetRotation().Rotator();
			const FRotator LastRotation = LastBoneTransform.GetRotation().Rotator();
			const FRotator DeltaRotation(FMath::Abs(Rotation.Pitch - LastRotation.Pitch), FMath::Abs(Rotation.Yaw - LastRotation.Yaw), FMath::Abs(Rotation.Roll - LastRotation.Roll));
			Track.DeltaRotationTrack.Add(DeltaRotation);
			Track.TotalRotation = Track.TotalRotation + DeltaRotation;
		}
	}
}

void URootMotionModifier_AdjustmentBlendWarp::AdjustmentBlendWarp(const FBoneContainer& BoneContainer, const FCSPose<FCompactPose>& AdditivePose, const FMotionDeltaTrackContainer& MotionDeltaTracks, FAnimSequenceTrackContainer& Output)
{
	auto CalculateAdditive = [](const FVector& Total, const FVector& Delta, const FVector& Additive, const FVector& PreviousAdditive, float Alpha)
	{
		FVector CurrentAdditive = FVector::ZeroVector;
		for (int32 Idx = 0; Idx < 3; Idx++)
		{
			if (!FMath::IsNearlyZero(Total[Idx], (FVector::FReal)1.f))
			{
				const FVector::FReal Percent = Delta[Idx] / Total[Idx];
				const FVector::FReal AdditiveDelta = FMath::Abs(Additive[Idx]) * Percent;
				CurrentAdditive[Idx] = (Additive[Idx] > 0.f) ? PreviousAdditive[Idx] + AdditiveDelta : PreviousAdditive[Idx] - AdditiveDelta;
			}
			else
			{
				CurrentAdditive[Idx] = Additive[Idx] * Alpha;
			}
		}
		return CurrentAdditive;
	};

	Output.Initialize(BoneContainer.GetCompactPoseNumBones());

	for (const FCompactPoseBoneIndex PoseBoneIndex : AdditivePose.GetPose().ForEachBoneIndex())
	{
		const FTransform& AdditiveTransform = AdditivePose.GetPose()[PoseBoneIndex];
		if (AdditiveTransform.Equals(FTransform::Identity))
		{
			continue;
		}

		const int32 BoneIndex = PoseBoneIndex.GetInt();
		const FMotionDeltaTrack& MotionDeltaTrack = MotionDeltaTracks.Tracks[BoneIndex];
		const int32 TotalFrames = MotionDeltaTrack.BoneTransformTrack.Num();

		FRawAnimSequenceTrack& Track = Output.AnimationTracks[BoneIndex];

		Track.PosKeys.Reserve(TotalFrames);
		Track.RotKeys.Reserve(TotalFrames);

		Track.PosKeys.Add(FVector3f(MotionDeltaTrack.BoneTransformTrack[0].GetLocation()));
		Track.RotKeys.Add(FQuat4f(MotionDeltaTrack.BoneTransformTrack[0].GetRotation()));

		Track.ScaleKeys.Add(FVector3f(1));

		Output.TrackNames[BoneIndex] = BoneContainer.GetReferenceSkeleton().GetBoneName(BoneContainer.GetBoneIndicesArray()[BoneIndex]);

		const FVector TotalAdditiveTranslation = AdditiveTransform.GetTranslation();
		const FVector TotalAdditiveRotation = AdditiveTransform.GetRotation().Rotator().Euler();

		FVector PrevAdditiveTranslation = FVector::ZeroVector;
		FVector PrevAdditiveRotation = FVector::ZeroVector;
		for (int32 FrameIdx = 1; FrameIdx < TotalFrames; FrameIdx++)
		{
			const FTransform& BoneTransform = MotionDeltaTrack.BoneTransformTrack[FrameIdx];
			const float Alpha = (FrameIdx / (float)(TotalFrames - 1));

			// Translation Key
			const FVector DeltaTranslation = MotionDeltaTrack.DeltaTranslationTrack[FrameIdx];
			const FVector CurrentAdditiveTranslation = CalculateAdditive(MotionDeltaTrack.TotalTranslation, DeltaTranslation, TotalAdditiveTranslation, PrevAdditiveTranslation, Alpha);
			Output.AnimationTracks[BoneIndex].PosKeys.Add(FVector3f(BoneTransform.GetTranslation() + CurrentAdditiveTranslation));
			PrevAdditiveTranslation = CurrentAdditiveTranslation;

			// Rotation Key
			const FVector DeltaRotation = MotionDeltaTrack.DeltaRotationTrack[FrameIdx].Euler();
			const FVector CurrentAdditiveRotation = CalculateAdditive(MotionDeltaTrack.TotalRotation.Euler(), DeltaRotation, TotalAdditiveRotation, PrevAdditiveRotation, Alpha);
			Output.AnimationTracks[BoneIndex].RotKeys.Add(FQuat4f(FRotator::MakeFromEuler(BoneTransform.GetRotation().Rotator().Euler() + CurrentAdditiveRotation).Quaternion()));
			PrevAdditiveRotation = CurrentAdditiveRotation;
		}
	}
}

FTransform URootMotionModifier_AdjustmentBlendWarp::ExtractWarpedRootMotion() const
{
	FTransform StartRootTransform;
	ExtractBoneTransformAtTime(StartRootTransform, 0, PreviousPosition);

	FTransform EndRootTransform;
	ExtractBoneTransformAtTime(EndRootTransform, 0, CurrentPosition);

	return EndRootTransform.GetRelativeTransform(StartRootTransform);
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
void URootMotionModifier_AdjustmentBlendWarp::DrawDebugWarpedTracks(float DrawDuration) const
{
	const UWorld* World = nullptr;
	const USkeletalMeshComponent* SkelMesh = nullptr;

	if (const UMotionWarpingBaseAdapter* OwnerAdapter = GetOwnerAdapter())
	{
		World = OwnerAdapter->GetActor()->GetWorld();
		SkelMesh = OwnerAdapter->GetMesh();
	}

	if (World && Result.GetNum() > 0 && PreviousPosition <= EndTime)
	{
		FTransform RootStartTransform;
		ExtractBoneTransformAtFrame(RootStartTransform, 0, 0);

		for (int32 TrackIndex = 0; TrackIndex < Result.GetNum(); TrackIndex++)
		{
			const int32 TotalFrames = Result.AnimationTracks[TrackIndex].PosKeys.Num();
			if (TotalFrames > 1)
			{
				for (int32 FrameIdx = 0; FrameIdx < Result.AnimationTracks[TrackIndex].PosKeys.Num() - 1; FrameIdx++)
				{
					FTransform T1;
					ExtractBoneTransformAtFrame(T1, TrackIndex, FrameIdx);
					T1 = T1 * RootStartTransform.Inverse() * CachedMeshTransform;

					FTransform T2;
					ExtractBoneTransformAtFrame(T2, TrackIndex, FrameIdx + 1);
					T2 = T2 * RootStartTransform.Inverse() * CachedMeshTransform;

					DrawDebugLine(World, T1.GetLocation(), T2.GetLocation(), FColor::Yellow, false, DrawDuration, 0, 0.5f);
				}
			}
		}

		const FTransform& MeshTransform = SkelMesh->GetComponentTransform();
		DrawDebugCoordinateSystem(World, CachedMeshTransform.GetLocation(), CachedMeshTransform.Rotator(), 20.f, false, DrawDuration, 0, 1.f);
		DrawDebugCoordinateSystem(World, MeshTransform.GetLocation(), MeshTransform.Rotator(), 20.f, false, DrawDuration, 0, 1.f);
		DrawDebugCoordinateSystem(World, GetTargetLocation(), GetTargetRotator(), 50.f, false, DrawDuration, 0, 1.f);
		DrawDebugCoordinateSystem(World, (CachedRootMotion * CachedMeshTransform).GetLocation(), (CachedRootMotion * CachedMeshTransform).Rotator(), 50.f, false, DrawDuration, 0, 1.f);
	}
}
#endif

void URootMotionModifier_AdjustmentBlendWarp::GetIKBoneTransformAndAlpha(FName BoneName, FTransform& OutTransform, float& OutAlpha) const
{
	if (Result.GetNum() == 0 || !bWarpIKBones || !IKBones.Contains(BoneName))
	{
		OutTransform = FTransform::Identity;
		OutAlpha = 0.f;
		return;
	}

	FTransform RootPrevPosition;
	ExtractBoneTransformAtTime(RootPrevPosition, 0, 0.f);

	FTransform BoneTransform;
	ExtractBoneTransformAtTime(BoneTransform, BoneName, PreviousPosition);

	OutTransform = BoneTransform * RootPrevPosition.Inverse() * CachedMeshTransform;
	OutAlpha = Weight;
}

void URootMotionModifier_AdjustmentBlendWarp::GetAdjustmentBlendIKBoneTransformAndAlpha(ACharacter* Character, FName BoneName, FTransform& OutTransform, float& OutAlpha)
{
	OutTransform = FTransform::Identity;
	OutAlpha = 0.f;

	const UMotionWarpingComponent* MotionWarpingComp = Character ? Character->FindComponentByClass<UMotionWarpingComponent>() : nullptr;
	if (MotionWarpingComp)
	{
		for (URootMotionModifier* Modifier : MotionWarpingComp->GetModifiers())
		{
			if (Modifier && Modifier->GetState() == ERootMotionModifierState::Active)
			{
				if (URootMotionModifier_AdjustmentBlendWarp* AdjustmentBlendWarpModifier = Cast<URootMotionModifier_AdjustmentBlendWarp>(Modifier))
				{
					// We must check if the Animation for the modifier is still relevant because in RootMotionFromMontageOnlyMode the montage could be aborted 
					// but the modifier will remain in the list until the next update
					const FAnimMontageInstance* RootMotionMontageInstance = Character->GetRootMotionAnimMontageInstance();
					const UAnimMontage* Montage = RootMotionMontageInstance ? ToRawPtr(RootMotionMontageInstance->Montage) : nullptr;
					if (Modifier->Animation == Montage)
					{
						AdjustmentBlendWarpModifier->GetIKBoneTransformAndAlpha(BoneName, OutTransform, OutAlpha);
						return;
					}
				}
			}
		}
	}
}

URootMotionModifier_AdjustmentBlendWarp* URootMotionModifier_AdjustmentBlendWarp::AddRootMotionModifierAdjustmentBlendWarp(UMotionWarpingComponent* InMotionWarpingComp, const UAnimSequenceBase* InAnimation, float InStartTime, float InEndTime,
	FName InWarpTargetName, EWarpPointAnimProvider InWarpPointAnimProvider, FTransform InWarpPointAnimTransform, FName InWarpPointAnimBoneName,
	bool bInWarpTranslation, bool bInIgnoreZAxis, bool bInWarpRotation, bool bInWarpIKBones, const TArray<FName>& InIKBones)
{
	if (ensureAlways(InMotionWarpingComp))
	{
		URootMotionModifier_AdjustmentBlendWarp* NewModifier = NewObject<URootMotionModifier_AdjustmentBlendWarp>(InMotionWarpingComp);
		NewModifier->Animation = InAnimation;
		NewModifier->StartTime = InStartTime;
		NewModifier->EndTime = InEndTime;
		NewModifier->WarpTargetName = InWarpTargetName;
		NewModifier->WarpPointAnimProvider = InWarpPointAnimProvider;
		NewModifier->WarpPointAnimTransform = InWarpPointAnimTransform;
		NewModifier->WarpPointAnimBoneName = InWarpPointAnimBoneName;
		NewModifier->bWarpTranslation = bInWarpTranslation;
		NewModifier->bIgnoreZAxis = bInIgnoreZAxis;
		NewModifier->bWarpRotation = bInWarpRotation;
		NewModifier->bWarpIKBones = bInWarpIKBones;
		NewModifier->IKBones = InIKBones;

		InMotionWarpingComp->AddModifier(NewModifier);

		return NewModifier;
	}

	return nullptr;
}
