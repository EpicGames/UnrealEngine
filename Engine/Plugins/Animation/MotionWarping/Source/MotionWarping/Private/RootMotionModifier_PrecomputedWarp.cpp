// Copyright Epic Games, Inc. All Rights Reserved.

#include "RootMotionModifier_PrecomputedWarp.h"
#include "DrawDebugHelpers.h"
#include "GameFramework/Character.h"
#include "Components/CapsuleComponent.h"
#include "MotionWarpingComponent.h"
#include "MotionWarpingAdapter.h"
#include "SceneView.h"
#include "VisualLogger/VisualLogger.h"

#include "AHEasing/easing.h"
#include "Animation/AnimMontage.h"

#if WITH_EDITOR
#include "Animation/AnimInstance.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "AnimPreviewInstance.h"
#include "PrimitiveDrawingUtils.h"
#include "BonePose.h"
#include "Engine/Font.h"
#include "CanvasTypes.h"
#include "Animation/AnimSequenceHelpers.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(RootMotionModifier_PrecomputedWarp)

URootMotionModifier_PrecomputedWarp::URootMotionModifier_PrecomputedWarp(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

namespace
{
	// This system bakes a warped root motion curve at a this fixed framerate, and then samples it and interpolates to the actual framerate to align
	constexpr float SamplingFrameTime = 1.0f/30.0f;

	FTransform ExtractRootMotionHelper(const UAnimSequenceBase* AnimationAsset, float CurrentTime, float DeltaTime, bool bLoop)
	{
		const FAnimExtractContext Context(static_cast<double>(CurrentTime), true, FDeltaTimeRecord(DeltaTime), bLoop);
		if (const UAnimMontage* Montage = Cast<UAnimMontage>(AnimationAsset))
		{
			return Montage->ExtractRootMotionFromTrackRange(CurrentTime, CurrentTime + DeltaTime, Context);
		}
		else
		{
			return AnimationAsset->ExtractRootMotion(Context);
		}
	}
	
	void GetTransformForFrame(float Frame, const TArray<FTransform>& Trajectory, FTransform& OutTransform)
	{
		check(!Trajectory.IsEmpty())
		
		int32 LowerFrame = FMath::Clamp(floor(Frame), 0, Trajectory.Num() -1);
		int32 UpperFrame = FMath::Min(Trajectory.Num() -1 , LowerFrame + 1);
		float Alpha = Frame - LowerFrame;

		OutTransform = Trajectory[LowerFrame];
		OutTransform.BlendWith(Trajectory[UpperFrame], Alpha);
	}

	float SampleCurve(float SampleTime, float StartTime, float EndTime, const TArray<float>& CurveData)
	{
		check(!CurveData.IsEmpty())
		
		if (SampleTime <= StartTime)
		{
			return 0.0f;
		}
		if (SampleTime >= EndTime)
		{
			return 1.0;
		}
		
		// do nearest neighbor sampling for simplicity and performance
		const int StartFrame = FMath::Clamp(round(StartTime / SamplingFrameTime), 0, CurveData.Num()-1);
		const int EndFrame = FMath::Clamp(round(EndTime / SamplingFrameTime), 0, CurveData.Num()-1);
		const int SampleFrame = FMath::Clamp(round(SampleTime / SamplingFrameTime), 0, CurveData.Num()-1);

		return (CurveData[SampleFrame] - CurveData[StartFrame])/(CurveData[EndFrame] - CurveData[StartFrame]);
	}
}
	
float URootMotionModifier_PrecomputedWarp::GetWeight(float CurrentTime, const FPrecomputedWarpCurve& WarpCurve) const
{
	float Result;

	if (WarpCurve.CurveType == EPrecomputedWarpWeightCurveType::FromRootMotionTranslation && !AnimTrajectoryData.TranslationCurve.IsEmpty())
	{
		const float Duration = (EndTime - StartTime);
		const float StartCurveSampleTime = FMath::Max(WarpCurve.StartRatio * Duration, ActualStartTime) - ActualStartTime;
		return SampleCurve(CurrentTime - ActualStartTime, StartCurveSampleTime, WarpCurve.EndRatio * Duration - ActualStartTime, AnimTrajectoryData.TranslationCurve);
	}
	else if (WarpCurve.CurveType == EPrecomputedWarpWeightCurveType::FromRootMotionRotation && !AnimTrajectoryData.RotationCurve.IsEmpty())
	{
		const float Duration = (EndTime - StartTime);
		const float StartCurveSampleTime = FMath::Max(WarpCurve.StartRatio * Duration, ActualStartTime) - ActualStartTime;
		return SampleCurve(CurrentTime - ActualStartTime, StartCurveSampleTime, WarpCurve.EndRatio * Duration - ActualStartTime, AnimTrajectoryData.RotationCurve);
	}
	else
	{
		const float StartTimeWithRatio = FMath::Max(FMath::Lerp(StartTime, EndTime, WarpCurve.StartRatio), ActualStartTime);
		const float EndTimeWithRatio = FMath::Lerp(StartTime, EndTime, WarpCurve.EndRatio);
		
		float CurrentRelativeTime;
		if (StartTimeWithRatio < EndTimeWithRatio - UE_SMALL_NUMBER)
		{
			CurrentRelativeTime = FMath::Clamp((CurrentTime - StartTimeWithRatio) / (EndTimeWithRatio - StartTimeWithRatio), 0, 1);
		}
		else
		{
			CurrentRelativeTime = 1.f;
		}
		
		switch(WarpCurve.CurveType)
		{
			case EPrecomputedWarpWeightCurveType::EaseIn:
				Result = CubicEaseIn(CurrentRelativeTime);
				break;
			case EPrecomputedWarpWeightCurveType::EaseOut:
				Result = CubicEaseOut(CurrentRelativeTime);
				break;
			case EPrecomputedWarpWeightCurveType::EaseInOut:
				Result = CubicEaseInOut(CurrentRelativeTime);
				break;
			case EPrecomputedWarpWeightCurveType::Instant:
				Result = 1.f;
				break;
			case EPrecomputedWarpWeightCurveType::DoNotWarp:
				Result = 0.f;
				break;
			default:
				Result = CurrentRelativeTime;
		}
	}

	return Result;
}

FTransform URootMotionModifier_PrecomputedWarp::ProcessRootMotion(const FTransform& InRootMotion, float DeltaSeconds)
{
	float CurrentTime = CurrentPosition;
	URootMotionModifier_PrecomputedWarp* AlignmentNotify = this;
	const UAnimSequenceBase* AnimationAsset = Animation.Get();

	const UMotionWarpingBaseAdapter* OwnerAdapter = GetOwnerAdapter();
	const AActor* OwnerAsActor = nullptr;

	FTransform CurrentTransform;
	FQuat MeshOffsetRotation;
	UObject* VLogObject = this;
	
	if (OwnerAdapter)
	{
		MeshOffsetRotation = OwnerAdapter->GetBaseVisualRotationOffset();
		FVector CurrentLocation       = OwnerAdapter->GetVisualRootLocation();
		FQuat CurrentRotation         = OwnerAdapter->GetActor()->GetActorQuat() * MeshOffsetRotation;
		CurrentTransform = FTransform(CurrentRotation, CurrentLocation);
		VLogObject = OwnerAdapter->GetActor();
	}

	if (bFirstFrame)
	{
		bFirstFrame = false;
		
		ActualStartTime = CurrentTime;
		RoundedEndTime = ActualStartTime + FMath::CeilToDouble((EndTime - ActualStartTime) / SamplingFrameTime) * SamplingFrameTime;
		
		TargetTransform = FTransform(GetTargetRotation() * MeshOffsetRotation, GetTargetLocation());
		
		// get alignment transform
		const float PredictionDelta = RoundedEndTime - CurrentTime;

		TargetTransform = AlignmentNotify->AlignOffset * TargetTransform;

		if (AlignmentNotify->bForceTargetTransformUpright)
		{
			FRotator Rotation = TargetTransform.GetRotation().Rotator();
			Rotation.Roll = 0;
			Rotation.Pitch = 0;
			TargetTransform.SetRotation(Rotation.Quaternion());
		}
		
		static const FBox UnitBox(FVector(-10, -10, -10), FVector(10, 10, 10));
		UE_VLOG_OBOX(VLogObject, "Alignment", Display, UnitBox, TargetTransform.ToMatrixWithScale(), FColor::Blue, TEXT(""));

		StartingRootTransform = CurrentTransform;
		
		// extract root motion trajectory todo: this should be cached and reused
		
		int NumFrames = 1 + (RoundedEndTime - ActualStartTime) / SamplingFrameTime;

		if (NumFrames <= 0)
		{
			return InRootMotion;
		}

		AnimTrajectoryData.Trajectory.SetNum(NumFrames);
		AnimTrajectoryData.TranslationCurve.SetNum(NumFrames);
		AnimTrajectoryData.RotationCurve.SetNum(NumFrames);
		
		float SteeringAngleThreshold = FMath::DegreesToRadians(AlignmentNotify->SteeringSettings.AngleThreshold);
		
		FTransform PredictedTransform;
		float PredictionTime = CurrentTime;
			
		AnimTrajectoryData.Trajectory[0] = PredictedTransform;
		PredictionTime += SamplingFrameTime;
		AnimTrajectoryData.TranslationCurve[0] = 0;

		for(int i=1;i<NumFrames; i++)
		{
			FTransform RootMotionThisFrame = ExtractRootMotionHelper(AnimationAsset, PredictionTime, SamplingFrameTime, false);
			
			PredictedTransform = RootMotionThisFrame * PredictedTransform;
			AnimTrajectoryData.Trajectory[i] = PredictedTransform;
			PredictionTime += SamplingFrameTime;

			AnimTrajectoryData.TranslationCurve[i] = RootMotionThisFrame.GetTranslation().Length() + AnimTrajectoryData.TranslationCurve[i-1]; 
			AnimTrajectoryData.RotationCurve[i] = fabs(RootMotionThisFrame.GetRotation().GetAngle()) + AnimTrajectoryData.RotationCurve[i-1]; 
		}

		// normalize curves;
		if (AnimTrajectoryData.TranslationCurve.Last() < UE_SMALL_NUMBER)
		{
			AnimTrajectoryData.TranslationCurve.Reset();
		}
		else
		{
			for(float& Value : AnimTrajectoryData.TranslationCurve)
			{
				Value /= AnimTrajectoryData.TranslationCurve.Last();
			}
		}
		
		if (AnimTrajectoryData.RotationCurve.Last() < UE_SMALL_NUMBER)
		{
			AnimTrajectoryData.RotationCurve.Reset();
		}
		else
		{
			for(float& Value : AnimTrajectoryData.RotationCurve)
			{
				Value /= AnimTrajectoryData.RotationCurve.Last();
			}
		}

		FVector UnWarpedPreviousPosition;
		FVector WarpedPreviousPosition;

		if (AlignmentNotify->bEnableSteering && AlignmentNotify->SteeringSettings.bEnableSmoothing)
		{
			FilteredSteeringTarget = FQuat::Identity;
			TargetSmoothingState.Reset();

			FilteredSteeringTarget = UKismetMathLibrary::QuaternionSpringInterp(FilteredSteeringTarget, FQuat::Identity, TargetSmoothingState,
											AlignmentNotify->SteeringSettings.SmoothStiffness, AlignmentNotify->SteeringSettings.SmoothDamping, DeltaSeconds, 1, 0, true);
		}

		WarpedTrajectory.SetNum(AnimTrajectoryData.Trajectory.Num());
			
		FVector MovementDirection = CurrentTransform.GetRotation().GetRightVector(); // todo, get this from translation over first few frames or based on some setting?
		
		FTransform InverseLastFrame = AnimTrajectoryData.Trajectory.Last().Inverse();
		// Translation warping + steering
		for(int i=0;i<NumFrames; i++)
		{
			FTransform TransformFromRoot = AnimTrajectoryData.Trajectory[i] * CurrentTransform;
			FTransform TransformFromTarget = AnimTrajectoryData.Trajectory[i] * InverseLastFrame * TargetTransform;
			
			FVector OldPosition = TransformFromRoot.GetTranslation();
			FVector UnWarpedDelta = OldPosition - UnWarpedPreviousPosition;
			UnWarpedPreviousPosition = OldPosition;

			FVector NewPosition = OldPosition;
				
			if (AlignmentNotify->bSeparateTranslationCurves)
			{
				float InMovementDirectionWeight = GetWeight(ActualStartTime + SamplingFrameTime*i, AlignmentNotify->TranslationWarpingCurve_InMovementDirection);
				float OutOfMovementDirectionWeight = GetWeight(ActualStartTime + SamplingFrameTime*i, AlignmentNotify->TranslationWarpingCurve_OutOfMovementDirection);

				FVector Delta = TransformFromTarget.GetTranslation() - OldPosition;
				FVector InMovementDirectionDelta = MovementDirection * Delta.Dot(MovementDirection);
				FVector OutOfMovementDirectionDelta = Delta - InMovementDirectionDelta;

				NewPosition = OldPosition + InMovementDirectionDelta * InMovementDirectionWeight + OutOfMovementDirectionDelta * OutOfMovementDirectionWeight;
			}
			else
			{
				float WarpWeight = GetWeight(ActualStartTime + SamplingFrameTime*i, AlignmentNotify->TranslationWarpingCurve);
				NewPosition = FMath::Lerp(OldPosition, TransformFromTarget.GetTranslation(), WarpWeight);
			}

			FVector WarpedDelta = NewPosition - WarpedPreviousPosition;
			WarpedPreviousPosition = NewPosition;

			WarpedTrajectory[i].SetTranslation(NewPosition);
			WarpedTrajectory[i].SetRotation(TransformFromRoot.GetRotation());

			if (i > 0 && AlignmentNotify->bEnableSteering)
			{
				FQuat OldRotation = TransformFromRoot.GetRotation();
				FQuat DirectionChange = FQuat::FindBetweenVectors(FVector(UnWarpedDelta.X, UnWarpedDelta.Y, 0) , FVector(WarpedDelta.X, WarpedDelta.Y, 0));

				if (AlignmentNotify->SteeringSettings.bEnableSmoothing)
				{
					if (DirectionChange.GetAngle() < SteeringAngleThreshold)
					{
						FilteredSteeringTarget = UKismetMathLibrary::QuaternionSpringInterp(FilteredSteeringTarget, DirectionChange, TargetSmoothingState,
														AlignmentNotify->SteeringSettings.SmoothStiffness, AlignmentNotify->SteeringSettings.SmoothDamping, DeltaSeconds, 1, 0, true);
					}
					
					DirectionChange = FilteredSteeringTarget;
					WarpedTrajectory[i].SetRotation(OldRotation*DirectionChange);
				}
				else
				{
					if (DirectionChange.GetAngle() < SteeringAngleThreshold)
					{
						WarpedTrajectory[i].SetRotation(OldRotation*DirectionChange);
					}
				}
			}
		}
		
		// Rotation warping
		for(int i=0;i<NumFrames; i++)
		{
			float WarpWeight = GetWeight(ActualStartTime + SamplingFrameTime*i, AlignmentNotify->RotationWarpingCurve);
			
			FQuat OldRotation = WarpedTrajectory[i].GetRotation();
			FTransform TransformFromTarget = AnimTrajectoryData.Trajectory[i] * InverseLastFrame * TargetTransform;
			
			WarpedTrajectory[i].SetRotation(FQuat::Slerp(OldRotation, TransformFromTarget.GetRotation(), WarpWeight));
		}
	}

	if (!WarpedTrajectory.IsEmpty())
	{
		float Frame = (CurrentTime + DeltaSeconds - ActualStartTime) / SamplingFrameTime;
		
		if (PreviousFrame < WarpedTrajectory.Num() - 1)
		{
			FTransform WorldTransform;
			
			if (Frame > WarpedTrajectory.Num()-1)
			{
				// when some time has elapsed past the target alignment time, play out regular root motion after the final aligned transform for that extra amount of time.
				float PostAlignmentTime = (Frame - (WarpedTrajectory.Num() - 1)) * SamplingFrameTime;
				FTransform PostAlignmentRootMotion = ExtractRootMotionHelper(AnimationAsset, RoundedEndTime, PostAlignmentTime, false);
				GetTransformForFrame(Frame, WarpedTrajectory, WorldTransform);
				WorldTransform = PostAlignmentRootMotion * WarpedTrajectory.Last();
			}
			else
			{
				GetTransformForFrame(Frame, WarpedTrajectory, WorldTransform);
			}

			FTransform OutRootMotion;
			
			if(AlignmentNotify->UpdateMode == EPrecomputedWarpUpdateMode::World)
			{
				OutRootMotion = WorldTransform.GetRelativeTransform(CurrentTransform);
			}
			else // relative mode
			{
				FTransform PrevTransform;
				GetTransformForFrame(PreviousFrame, WarpedTrajectory, PrevTransform);
				OutRootMotion = WorldTransform.GetRelativeTransform(PrevTransform);
			}
	
			// unwarped trajectory relative to starting transform
			FTransform PreviousTransform = AnimTrajectoryData.Trajectory[0] * StartingRootTransform;
			for(int i=1;i<AnimTrajectoryData.Trajectory.Num(); i++)
			{
				FTransform TransformFromRoot = AnimTrajectoryData.Trajectory[i] * StartingRootTransform;
				UE_VLOG_SEGMENT(VLogObject, "Alignment", Display, PreviousTransform.GetLocation(), TransformFromRoot.GetLocation(), i%2 == 0 ? FColor::Yellow : FColor::Red, TEXT(""));
				PreviousTransform = TransformFromRoot;
			}
	
			// unwarped trajectory relative to target transform
			FTransform InverseLastFrame = AnimTrajectoryData.Trajectory.Last().Inverse();
			PreviousTransform = AnimTrajectoryData.Trajectory[0] * InverseLastFrame * TargetTransform;
			for(int i=1;i<AnimTrajectoryData.Trajectory.Num(); i++)
			{
				FTransform TransformFromTarget = AnimTrajectoryData.Trajectory[i] * InverseLastFrame * TargetTransform;
				UE_VLOG_SEGMENT(VLogObject, "Alignment", Display, PreviousTransform.GetLocation(), TransformFromTarget.GetLocation(), i%2 == 0 ? FColor::Yellow : FColor::Red, TEXT(""));
				PreviousTransform = TransformFromTarget;
			}
	
			// the warped trajectory
			for(int i=1; i<WarpedTrajectory.Num(); i++)
			{
				UE_VLOG_SEGMENT(VLogObject, "Alignment", Display, WarpedTrajectory[i-1].GetLocation(), WarpedTrajectory[i].GetLocation(), i%2 == 0 ? FColor::Green : FColor::Blue, TEXT(""));
			}

			// a dot representing our current root position
			UE_VLOG_SPHERE(VLogObject, "Alignment", Display, CurrentTransform.GetLocation(),  1, FColor::Green, TEXT(""));
			
			// a dot representing our current target position on the trajectory 
			UE_VLOG_SPHERE(VLogObject, "Alignment", Display, WorldTransform.GetLocation(),  1, FColor::Red, TEXT(""));
			
			PreviousFrame = Frame;

			return OutRootMotion;
		}
	}
	
	
	return InRootMotion;
}

TSharedPtr<FRootMotionModifierSwapState> URootMotionModifier_PrecomputedWarp::GetSwapState() const
{
	TSharedPtr<FRootMotionModifierSwapState> Result = MakeShared<FRootMotionModifier_PrecomputedWarpSwapState>();
	PopulateSwapState(Result);
	return Result;
}

void URootMotionModifier_PrecomputedWarp::PopulateSwapState(TSharedPtr<FRootMotionModifierSwapState> InSwapState) const
{
	Super::PopulateSwapState(InSwapState);

	TSharedPtr<FRootMotionModifier_PrecomputedWarpSwapState> PrecomputedWarpSwapState = StaticCastSharedPtr<FRootMotionModifier_PrecomputedWarpSwapState>(InSwapState);

	PrecomputedWarpSwapState->bFirstFrame = bFirstFrame;
	PrecomputedWarpSwapState->ActualStartTime = ActualStartTime;
	PrecomputedWarpSwapState->RoundedEndTime = RoundedEndTime;
	PrecomputedWarpSwapState->PreviousFrame = PreviousFrame;
	PrecomputedWarpSwapState->StartingRootTransform = StartingRootTransform;
	PrecomputedWarpSwapState->TargetTransform = TargetTransform;
	PrecomputedWarpSwapState->FilteredSteeringTarget = FilteredSteeringTarget;
	PrecomputedWarpSwapState->TargetSmoothingState = TargetSmoothingState;
	PrecomputedWarpSwapState->WarpedTrajectory = WarpedTrajectory;
	PrecomputedWarpSwapState->AnimTrajectoryData = AnimTrajectoryData;
}

void URootMotionModifier_PrecomputedWarp::RestoreFromSwapState(TSharedPtr<FRootMotionModifierSwapState> InSwapState)
{
	Super::RestoreFromSwapState(InSwapState);

	TSharedPtr<FRootMotionModifier_PrecomputedWarpSwapState> PrecomputedWarpSwapState = StaticCastSharedPtr<FRootMotionModifier_PrecomputedWarpSwapState>(InSwapState);

	bFirstFrame = PrecomputedWarpSwapState->bFirstFrame;
	ActualStartTime = PrecomputedWarpSwapState->ActualStartTime;
	RoundedEndTime = PrecomputedWarpSwapState->RoundedEndTime;
	PreviousFrame = PrecomputedWarpSwapState->PreviousFrame;
	StartingRootTransform = PrecomputedWarpSwapState->StartingRootTransform;
	TargetTransform = PrecomputedWarpSwapState->TargetTransform;
	FilteredSteeringTarget = PrecomputedWarpSwapState->FilteredSteeringTarget;
	TargetSmoothingState = PrecomputedWarpSwapState->TargetSmoothingState;
	WarpedTrajectory = PrecomputedWarpSwapState->WarpedTrajectory;
	AnimTrajectoryData = PrecomputedWarpSwapState->AnimTrajectoryData;
}