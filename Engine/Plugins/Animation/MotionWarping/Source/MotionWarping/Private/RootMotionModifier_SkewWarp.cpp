// Copyright Epic Games, Inc. All Rights Reserved.

#include "RootMotionModifier_SkewWarp.h"
#include "DrawDebugHelpers.h"
#include "GameFramework/Character.h"
#include "Components/CapsuleComponent.h"
#include "MotionWarpingComponent.h"
#include "MotionWarpingAdapter.h"
#include "MotionWarpingMontageTrajectoryAdapter.h"
#include "SceneView.h"

#if WITH_EDITOR
#include "Animation/AnimInstance.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "AnimPreviewInstance.h"
#include "PrimitiveDrawingUtils.h"
#include "BonePose.h"
#include "Engine/Engine.h"
#include "Engine/Font.h"
#include "CanvasTypes.h"
#include "Animation/AnimSequenceHelpers.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(RootMotionModifier_SkewWarp)

URootMotionModifier_SkewWarp::URootMotionModifier_SkewWarp(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FVector URootMotionModifier_SkewWarp::WarpTranslation(const FTransform& CurrentTransform, const FVector& DeltaTranslation, const FVector& TotalTranslation, const FVector& TargetLocation)
{
	if (!DeltaTranslation.IsNearlyZero())
	{
		const FQuat CurrentRotation = CurrentTransform.GetRotation();
		const FVector CurrentLocation = CurrentTransform.GetLocation();
		const FVector FutureLocation = CurrentLocation + TotalTranslation;
		const FVector CurrentToWorldOffset = TargetLocation - CurrentLocation;
		const FVector CurrentToRootOffset = FutureLocation - CurrentLocation;

		// Create a matrix we can use to put everything into a space looking straight at RootMotionSyncPosition. "forward" should be the axis along which we want to scale. 
		FVector ToRootNormalized = CurrentToRootOffset.GetSafeNormal();

		float BestMatchDot = FMath::Abs(FVector::DotProduct(ToRootNormalized, CurrentRotation.GetAxisX()));
		FMatrix ToRootSyncSpace = FRotationMatrix::MakeFromXZ(ToRootNormalized, CurrentRotation.GetAxisZ());

		float ZDot = FMath::Abs(FVector::DotProduct(ToRootNormalized, CurrentRotation.GetAxisZ()));
		if (ZDot > BestMatchDot)
		{
			ToRootSyncSpace = FRotationMatrix::MakeFromXZ(ToRootNormalized, CurrentRotation.GetAxisX());
			BestMatchDot = ZDot;
		}

		float YDot = FMath::Abs(FVector::DotProduct(ToRootNormalized, CurrentRotation.GetAxisY()));
		if (YDot > BestMatchDot)
		{
			ToRootSyncSpace = FRotationMatrix::MakeFromXZ(ToRootNormalized, CurrentRotation.GetAxisZ());
		}

		// Put everything into RootSyncSpace.
		const FVector RootMotionInSyncSpace = ToRootSyncSpace.InverseTransformVector(DeltaTranslation);
		const FVector CurrentToWorldSync = ToRootSyncSpace.InverseTransformVector(CurrentToWorldOffset);
		const FVector CurrentToRootMotionSync = ToRootSyncSpace.InverseTransformVector(CurrentToRootOffset);

		FVector CurrentToWorldSyncNorm = CurrentToWorldSync;
		CurrentToWorldSyncNorm.Normalize();

		FVector CurrentToRootMotionSyncNorm = CurrentToRootMotionSync;
		CurrentToRootMotionSyncNorm.Normalize();

		// Calculate skew Yaw Angle. 
		FVector FlatToWorld = FVector(CurrentToWorldSyncNorm.X, CurrentToWorldSyncNorm.Y, 0.0f);
		FlatToWorld.Normalize();
		FVector FlatToRoot = FVector(CurrentToRootMotionSyncNorm.X, CurrentToRootMotionSyncNorm.Y, 0.0f);
		FlatToRoot.Normalize();
		float AngleAboutZ = FMath::Acos(FVector::DotProduct(FlatToWorld, FlatToRoot));
		float AngleAboutZNorm = FMath::DegreesToRadians(FRotator::NormalizeAxis(FMath::RadiansToDegrees(AngleAboutZ)));
		if (FlatToWorld.Y < 0.0f)
		{
			AngleAboutZNorm *= -1.0f;
		}

		// Calculate Skew Pitch Angle. 
		FVector ToWorldNoY = FVector(CurrentToWorldSyncNorm.X, 0.0f, CurrentToWorldSyncNorm.Z);
		ToWorldNoY.Normalize();
		FVector ToRootNoY = FVector(CurrentToRootMotionSyncNorm.X, 0.0f, CurrentToRootMotionSyncNorm.Z);
		ToRootNoY.Normalize();
		const float AngleAboutY = FMath::Acos(FVector::DotProduct(ToWorldNoY, ToRootNoY));
		float AngleAboutYNorm = FMath::DegreesToRadians(FRotator::NormalizeAxis(FMath::RadiansToDegrees(AngleAboutY)));
		if (ToWorldNoY.Z < 0.0f)
		{
			AngleAboutYNorm *= -1.0f;
		}

		FVector SkewedRootMotion = FVector::ZeroVector;
		float ProjectedScale = FVector::DotProduct(CurrentToWorldSync, CurrentToRootMotionSyncNorm) / CurrentToRootMotionSync.Size();
		if (ProjectedScale != 0.0f)
		{
			FMatrix ScaleMatrix;
			ScaleMatrix.SetIdentity();
			ScaleMatrix.SetAxis(0, FVector(ProjectedScale, 0.0f, 0.0f));
			ScaleMatrix.SetAxis(1, FVector(0.0f, 1.0f, 0.0f));
			ScaleMatrix.SetAxis(2, FVector(0.0f, 0.0f, 1.0f));

			FMatrix ShearXAlongYMatrix;
			ShearXAlongYMatrix.SetIdentity();
			ShearXAlongYMatrix.SetAxis(0, FVector(1.0f, FMath::Tan(AngleAboutZNorm), 0.0f));
			ShearXAlongYMatrix.SetAxis(1, FVector(0.0f, 1.0f, 0.0f));
			ShearXAlongYMatrix.SetAxis(2, FVector(0.0f, 0.0f, 1.0f));

			FMatrix ShearXAlongZMatrix;
			ShearXAlongZMatrix.SetIdentity();
			ShearXAlongZMatrix.SetAxis(0, FVector(1.0f, 0.0f, FMath::Tan(AngleAboutYNorm)));
			ShearXAlongZMatrix.SetAxis(1, FVector(0.0f, 1.0f, 0.0f));
			ShearXAlongZMatrix.SetAxis(2, FVector(0.0f, 0.0f, 1.0f));

			FMatrix ScaledSkewMatrix = ScaleMatrix * ShearXAlongYMatrix * ShearXAlongZMatrix;

			// Skew and scale the Root motion. 
			SkewedRootMotion = ScaledSkewMatrix.TransformVector(RootMotionInSyncSpace);
		}
		else if (!CurrentToRootMotionSync.IsZero() && !CurrentToWorldSync.IsZero() && !RootMotionInSyncSpace.IsZero())
		{
			// Figure out ratio between remaining Root and remaining World. Then project scaled length of current Root onto World.
			const float Scale = CurrentToWorldSync.Size() / CurrentToRootMotionSync.Size();
			const float StepTowardTarget = RootMotionInSyncSpace.ProjectOnTo(RootMotionInSyncSpace).Size();
			SkewedRootMotion = CurrentToWorldSyncNorm * (Scale * StepTowardTarget);
		}

		// Put our result back in world space.  
		return ToRootSyncSpace.TransformVector(SkewedRootMotion);
	}

	return FVector::ZeroVector;
}

FTransform URootMotionModifier_SkewWarp::ProcessRootMotion(const FTransform& InRootMotion, float DeltaSeconds)
{
	const UMotionWarpingBaseAdapter* OwnerAdapter = GetOwnerAdapter();

	if (!OwnerAdapter)
	{
		return InRootMotion;
	}

	FTransform FinalRootMotion = InRootMotion;

	const FTransform RootMotionTotal = UMotionWarpingUtilities::ExtractRootMotionFromAnimation(Animation.Get(), PreviousPosition, EndTime);
	const FTransform RootMotionDelta = UMotionWarpingUtilities::ExtractRootMotionFromAnimation(Animation.Get(), PreviousPosition, FMath::Min(CurrentPosition, EndTime));

	FTransform ExtraRootMotion = FTransform::Identity;
	if (CurrentPosition > EndTime)
	{
		ExtraRootMotion = UMotionWarpingUtilities::ExtractRootMotionFromAnimation(Animation.Get(), EndTime, CurrentPosition);
	}

	if (bWarpTranslation)
	{
		if (bRootMotionPaused)
		{
			 FinalRootMotion.SetTranslation(FVector::ZeroVector);
		}
		else if (!bWarpingPaused)
		{
		const FVector CurrentLocation       = OwnerAdapter->GetVisualRootLocation();
		const FQuat CurrentRotation         = OwnerAdapter->GetCurrentRotation();
		FVector MeshTranslationOffset       = OwnerAdapter->GetBaseVisualTranslationOffset();
		if (!bWarpToFeetLocation)
		{
			MeshTranslationOffset.Z = 0.f;
		}

		const FQuat MeshRotationOffset      = OwnerAdapter->GetBaseVisualRotationOffset();

		const FVector DeltaTranslation = RootMotionDelta.GetLocation();
		const FVector TotalTranslation = RootMotionTotal.GetLocation();

		FVector TargetLocation = GetTargetLocation();
		if (bIgnoreZAxis)
		{
			TargetLocation.Z = CurrentLocation.Z;
		}

		// use a larger threshold than UE_KINDA_SMALL_NUMBER to ensure that when taking the animated codepath,
		// other small number thresholds inside WarpTranslation will not fail
		constexpr double RootMotionThreshold = 2e-4;

		// if there is translation in the animation, warp it
		if (!TotalRootMotionWithinWindow.GetTranslation().IsNearlyZero(RootMotionThreshold))
		{
			if (!DeltaTranslation.IsNearlyZero(RootMotionThreshold))
			{
				const FTransform MeshTransform = FTransform(MeshRotationOffset, MeshTranslationOffset) * OwnerAdapter->GetCurrentTransform();
				TargetLocation = MeshTransform.InverseTransformPositionNoScale(TargetLocation) - RootMotionRemainingAfterNotify.GetTranslation();

				if (bIgnoreZAxis)
				{
					TargetLocation.Z = 0;
				}

				FVector WarpedTranslation = WarpTranslation(FTransform::Identity, DeltaTranslation, TotalTranslation, TargetLocation) + ExtraRootMotion.GetLocation();

				if (MaxSpeedClampRatio > 0.0f)
				{
					if (bIgnoreZAxis)
					{
						const float AnimationSpeed = DeltaTranslation.Size2D();
						WarpedTranslation = WarpedTranslation.GetClampedToMaxSize2D(AnimationSpeed * MaxSpeedClampRatio);
					}
					else
					{
						const float AnimationSpeed = DeltaTranslation.Size();
						WarpedTranslation = WarpedTranslation.GetClampedToMaxSize(AnimationSpeed * MaxSpeedClampRatio);
					}
				}
				FinalRootMotion.SetTranslation(WarpedTranslation);
			}
		}
		// if there is no translation in the animation, add it
		else
		{
			const FVector DeltaToTarget = TargetLocation - CurrentLocation;
			if (DeltaToTarget.IsNearlyZero())
			{
				FinalRootMotion.SetTranslation(FVector::ZeroVector);
			}
			else
			{
				float Alpha = FMath::Clamp((CurrentPosition - ActualStartTime) / (EndTime - ActualStartTime), 0.f, 1.f);
				Alpha = FAlphaBlend::AlphaToBlendOption(Alpha, AddTranslationEasingFunc, AddTranslationEasingCurve);

				FVector NextLocation = FMath::Lerp<FVector, float>(StartTransform.GetLocation(), TargetLocation, Alpha);
				if (bIgnoreZAxis)
				{
					NextLocation.Z = CurrentLocation.Z;
				}

				FVector FinalDeltaTranslation = (NextLocation - CurrentLocation);
				FinalDeltaTranslation = (CurrentRotation.Inverse() * DeltaToTarget.ToOrientationQuat()).GetForwardVector() * FinalDeltaTranslation.Size();
				FinalDeltaTranslation = MeshRotationOffset.UnrotateVector(FinalDeltaTranslation);

				FinalRootMotion.SetTranslation(FinalDeltaTranslation + ExtraRootMotion.GetLocation());
			}
		}
	}
	}

	if(bWarpRotation)
	{
		const FQuat WarpedRotation = ExtraRootMotion.GetRotation() * WarpRotation(RootMotionDelta, RootMotionTotal, DeltaSeconds);
		FinalRootMotion.SetRotation(WarpedRotation);
	}

	// Debug
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	const int32 DebugLevel = FMotionWarpingCVars::CVarMotionWarpingDebug.GetValueOnAnyThread();
	if (DebugLevel == 1 || DebugLevel == 3)
	{
		PrintLog(TEXT("SkewWarp"), InRootMotion, FinalRootMotion);
	}

	if (DebugLevel == 2 || DebugLevel == 3)
	{
		const float DrawDebugDuration = FMotionWarpingCVars::CVarMotionWarpingDrawDebugDuration.GetValueOnGameThread();

		if (const AActor* OwnerAsActor = OwnerAdapter->GetActor())
		{
			DrawDebugCoordinateSystem(OwnerAsActor->GetWorld(), GetTargetLocation(), GetTargetRotator(), 50.f, false, DrawDebugDuration, 0, 1.f);
		}
	}
#endif

	return FinalRootMotion;
}

URootMotionModifier_SkewWarp* URootMotionModifier_SkewWarp::AddRootMotionModifierSkewWarp(UMotionWarpingComponent* InMotionWarpingComp, const UAnimSequenceBase* InAnimation, float InStartTime, float InEndTime,
	FName InWarpTargetName, EWarpPointAnimProvider InWarpPointAnimProvider, FTransform InWarpPointAnimTransform, FName InWarpPointAnimBoneName,
	bool bInWarpTranslation, bool bInIgnoreZAxis, bool bInWarpRotation, EMotionWarpRotationType InRotationType, EMotionWarpRotationMethod InRotationMethod, float InWarpRotationTimeMultiplier, float InWarpMaxRotationRate)
{
	if (ensureAlways(InMotionWarpingComp))
	{
		URootMotionModifier_SkewWarp* NewModifier = NewObject<URootMotionModifier_SkewWarp>(InMotionWarpingComp);
		NewModifier->Animation = InAnimation;
		NewModifier->StartTime = InStartTime;
		NewModifier->EndTime = InEndTime;
		NewModifier->WarpTargetName = InWarpTargetName;
		NewModifier->WarpPointAnimProvider = InWarpPointAnimProvider;
		NewModifier->WarpPointAnimTransform = InWarpPointAnimTransform;
		NewModifier->WarpPointAnimBoneName = InWarpPointAnimBoneName;
		NewModifier->bWarpTranslation = bInWarpTranslation;
		NewModifier->bIgnoreZAxis = bInIgnoreZAxis;
		NewModifier->bWarpRotation = bInWarpRotation;
		NewModifier->RotationType = InRotationType;
		NewModifier->RotationMethod = InRotationMethod;
		NewModifier->WarpRotationTimeMultiplier = InWarpRotationTimeMultiplier;
		NewModifier->WarpMaxRotationRate = InWarpMaxRotationRate;

		InMotionWarpingComp->AddModifier(NewModifier);

		return NewModifier;
	}

	return nullptr;
}

#if WITH_EDITOR

FTransform URootMotionModifier_SkewWarp::GetDebugWarpPointTransform(USkeletalMeshComponent* MeshComp, const UAnimSequenceBase* InAnimation, const UMirrorDataTable* MirrorTable, const float NotifyEndTime) const
{
	FTransform WarpPointTransform = FTransform::Identity;
	if (WarpPointAnimProvider == EWarpPointAnimProvider::None)
	{
		WarpPointTransform = UE::Anim::ExtractRootMotionFromAnimationAsset(InAnimation, MirrorTable, 0.0, NotifyEndTime);
	}
	else if (WarpPointAnimProvider == EWarpPointAnimProvider::Static)
	{
		// This method returns the warp transform relative to the actor location on the first frame of the animation.
		// The WarpPointAnimTransform is defined in the same coordinate space as the root motion track.
		// Adjust the return warp point to be relative to the first frame's Actor transform.
		const FTransform FirstFrameTransform = UE::Anim::ExtractRootTransformFromAnimationAsset(InAnimation, 0.0f);
		WarpPointTransform = WarpPointAnimTransform * FirstFrameTransform.Inverse();
	}
	else if (WarpPointAnimProvider == EWarpPointAnimProvider::Bone)
	{
		if (const UAnimInstance* AnimInstance = MeshComp->GetAnimInstance())
		{
			const FBoneContainer& FullBoneContainer = AnimInstance->GetRequiredBones();
			const int32 BoneIndex = FullBoneContainer.GetPoseBoneIndexForBoneName(WarpPointAnimBoneName);
			if (BoneIndex != INDEX_NONE)
			{
				TArray<FBoneIndexType> RequiredBoneIndexArray = { 0, (FBoneIndexType)BoneIndex };
				FullBoneContainer.GetReferenceSkeleton().EnsureParentsExistAndSort(RequiredBoneIndexArray);

				const FBoneContainer LimitedBoneContainer(RequiredBoneIndexArray, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::DisallowAll), *FullBoneContainer.GetAsset());

				FCSPose<FCompactPose> Pose;
				UMotionWarpingUtilities::ExtractComponentSpacePose(InAnimation, LimitedBoneContainer, NotifyEndTime, false, Pose);

				WarpPointTransform = Pose.GetComponentSpaceTransform(FCompactPoseBoneIndex(1));
			}
		}
	}
	return WarpPointTransform;
}

void URootMotionModifier_SkewWarp::DrawInEditor(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* MeshComp, const UAnimSequenceBase* InAnimation, const FAnimNotifyEvent& NotifyEvent) const
{
	constexpr float DepthBias = 4.0f;
	constexpr bool bScreenSpace = true;

	check(MeshComp);
	check(PDI);

	// Early out if the animation does not have root motion.
	if (!InAnimation || !InAnimation->HasRootMotion())
	{
		return;
	}
	
	const float NotifyStartTime = NotifyEvent.GetTriggerTime();
	const float NotifyEndTime = NotifyEvent.GetEndTriggerTime();

	const UMirrorDataTable* MirrorTable = nullptr;
	FTransform ReferenceTransform = FTransform::Identity;
	if (UDebugSkelMeshComponent* DebugMeshComp = Cast<UDebugSkelMeshComponent>(MeshComp))
	{
		ReferenceTransform = DebugMeshComp->RootMotionReferenceTransform; // Actors location at the beginning of the animation
		MirrorTable = DebugMeshComp->PreviewInstance ? DebugMeshComp->PreviewInstance->GetMirrorDataTable() : nullptr;
	}

	const FTransform WarpPointTransform = GetDebugWarpPointTransform(MeshComp, InAnimation, MirrorTable, NotifyEndTime) * ReferenceTransform;

	// Draw notify duration on root motion track.
	const FFrameRate FrameRate = InAnimation->GetSamplingFrameRate();

	const int32 StartFrame = FrameRate.AsFrameTime(NotifyStartTime).CeilToFrame().Value;
	const int32 EndFrame = FrameRate.AsFrameTime(NotifyEndTime).FloorToFrame().Value;

	const FTransform StartRootTransform = UE::Anim::ExtractRootMotionFromAnimationAsset(InAnimation, MirrorTable, 0.0, NotifyStartTime) * ReferenceTransform;
	const FTransform EndRootTransform = UE::Anim::ExtractRootMotionFromAnimationAsset(InAnimation, MirrorTable, 0.0, NotifyEndTime) * ReferenceTransform;

	constexpr double TrackOffset = 2.0; 
	
	FVector PrevLocation = StartRootTransform.GetLocation() + StartRootTransform.GetUnitAxis(EAxis::Z) * TrackOffset;
	for (int32 Frame = StartFrame; Frame <= EndFrame; Frame++)
	{
		const double Time = FMath::Clamp(FrameRate.AsSeconds(Frame), 0., (double)InAnimation->GetPlayLength());
		const FTransform Transform = UE::Anim::ExtractRootMotionFromAnimationAsset(InAnimation, MirrorTable, 0.0, Time) * ReferenceTransform;
		const FVector Location = Transform.GetLocation() + Transform.GetUnitAxis(EAxis::Z) * TrackOffset;
		
		PDI->DrawTranslucentLine(PrevLocation, Location, NotifyEvent.NotifyColor, SDPG_World, 1.5f, DepthBias, bScreenSpace);

		PrevLocation = Location;
	}

	const FVector EndLocation = EndRootTransform.GetLocation() + EndRootTransform.GetUnitAxis(EAxis::Z) * TrackOffset;
	PDI->DrawTranslucentLine(PrevLocation, EndLocation, NotifyEvent.NotifyColor, SDPG_World, 1.5f, DepthBias, bScreenSpace);
	PrevLocation = EndLocation;
	
	// Draw line connecting root motion segment to the warp target.
	if (FVector::Distance(PrevLocation, WarpPointTransform.GetLocation()) > 5.0)
	{
		DrawDashedLine(PDI, PrevLocation, WarpPointTransform.GetLocation(), NotifyEvent.NotifyColor, 5.0f, SDPG_World, DepthBias);
	}

	// Draw vertical ticks indicating start and end locations.
	constexpr double RangeTickSize = 5.0;
	PDI->DrawTranslucentLine(StartRootTransform.GetLocation(), StartRootTransform.GetLocation() + StartRootTransform.GetUnitAxis(EAxis::Z) * RangeTickSize, FColor::Black.WithAlpha(128), SDPG_World, 1, DepthBias, bScreenSpace);
	PDI->DrawTranslucentLine(EndRootTransform.GetLocation(), EndRootTransform.GetLocation() + EndRootTransform.GetUnitAxis(EAxis::Z) * RangeTickSize, FColor::Black.WithAlpha(128), SDPG_World, 1, DepthBias, bScreenSpace);
	
	// Draw warp target transform
	constexpr double WarpPointSize = 10.0;
	const FVector WarpLocation = WarpPointTransform.GetLocation();
	const FVector WarpAxisX = WarpPointTransform.GetUnitAxis(EAxis::X) * WarpPointSize;
	const FVector WarpAxisY = WarpPointTransform.GetUnitAxis(EAxis::Y) * WarpPointSize;
	const FVector WarpAxisZ = WarpPointTransform.GetUnitAxis(EAxis::Z) * WarpPointSize;
	PDI->DrawLine(WarpLocation, WarpLocation + WarpAxisX, FColor::Red, SDPG_Foreground, 1.0f, DepthBias, bScreenSpace);
	PDI->DrawLine(WarpLocation, WarpLocation + WarpAxisY, FColor::Green, SDPG_Foreground, 1.0f, DepthBias, bScreenSpace);
	PDI->DrawLine(WarpLocation, WarpLocation + WarpAxisZ, FColor::Blue, SDPG_Foreground, 1.0f, DepthBias, bScreenSpace);
	

}

void URootMotionModifier_SkewWarp::DrawCanvasInEditor(FCanvas& Canvas, FSceneView& View, USkeletalMeshComponent* MeshComp, const UAnimSequenceBase* InAnimation, const FAnimNotifyEvent& NotifyEvent) const
{
	// Early out if the animation does not have root motion.
	if (!InAnimation || !InAnimation->HasRootMotion())
	{
		return;
	}

	const float NotifyEndTime = NotifyEvent.GetEndTriggerTime();

	const UMirrorDataTable* MirrorTable = nullptr;
	FTransform ReferenceTransform = FTransform::Identity;
	if (UDebugSkelMeshComponent* DebugMeshComp = Cast<UDebugSkelMeshComponent>(MeshComp))
	{
		ReferenceTransform = DebugMeshComp->RootMotionReferenceTransform;
		MirrorTable = DebugMeshComp->PreviewInstance ? DebugMeshComp->PreviewInstance->GetMirrorDataTable() : nullptr;
	}

	const FTransform WarpPointTransform = GetDebugWarpPointTransform(MeshComp, InAnimation, MirrorTable, NotifyEndTime) * ReferenceTransform;

	if (!WarpTargetName.IsNone())
	{
		FVector2D PixelLocation;
		if (View.WorldToPixel(WarpPointTransform.GetLocation(),PixelLocation))
		{
			PixelLocation.X = FMath::RoundToFloat(PixelLocation.X);
			PixelLocation.Y = FMath::RoundToFloat(PixelLocation.Y);

			constexpr FColor LabelColor(200, 200, 200);
			constexpr FLinearColor ShadowColor(0, 0, 0, 0.3f);
			const UFont* SmallFont = GEngine->GetSmallFont();

			Canvas.DrawShadowedString(PixelLocation.X, PixelLocation.Y, *WarpTargetName.ToString(), SmallFont, NotifyEvent.NotifyColor, ShadowColor);
			PixelLocation.Y += SmallFont->GetMaxCharHeight();
		}
	}
}
#endif