// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "AnimNotifyState_MotionWarping.generated.h"

#define UE_API MOTIONWARPING_API

class UMotionWarpingComponent;
class UAnimSequenceBase;
class URootMotionModifier;

/** AnimNotifyState used to define a motion warping window in the animation */
UCLASS(MinimalAPI, meta = (DisplayName = "Motion Warping"))
class UAnimNotifyState_MotionWarping : public UAnimNotifyState
{
	GENERATED_BODY()

public:

	//@TODO: Prevent notify callbacks and add comments explaining why we don't use those here.

	UPROPERTY(EditAnywhere, Instanced, BlueprintReadWrite, Category = "Config")
	TObjectPtr<URootMotionModifier> RootMotionModifier;

	UE_API UAnimNotifyState_MotionWarping(const FObjectInitializer& ObjectInitializer);

	/** Called from the MotionWarpingComp when this notify becomes relevant. See: UMotionWarpingComponent::Update */
	UE_API void OnBecomeRelevant(UMotionWarpingComponent* MotionWarpingComp, const UAnimSequenceBase* Animation, float StartTime, float EndTime) const;

	/** Creates a root motion modifier from the config class defined in the notify */
	UFUNCTION(BlueprintNativeEvent, Category = "Motion Warping")
	UE_API URootMotionModifier* AddRootMotionModifier(UMotionWarpingComponent* MotionWarpingComp, const UAnimSequenceBase* Animation, float StartTime, float EndTime) const;

	UFUNCTION()
	UE_API void OnRootMotionModifierActivate(UMotionWarpingComponent* MotionWarpingComp, URootMotionModifier* Modifier);

	UFUNCTION()
	UE_API void OnRootMotionModifierUpdate(UMotionWarpingComponent* MotionWarpingComp, URootMotionModifier* Modifier);

	UFUNCTION()
	UE_API void OnRootMotionModifierDeactivate(UMotionWarpingComponent* MotionWarpingComp, URootMotionModifier* Modifier);

	UFUNCTION(BlueprintImplementableEvent, Category = "Motion Warping")
	UE_API void OnWarpBegin(UMotionWarpingComponent* MotionWarpingComp, URootMotionModifier* Modifier) const;

	UFUNCTION(BlueprintImplementableEvent, Category = "Motion Warping")
	UE_API void OnWarpUpdate(UMotionWarpingComponent* MotionWarpingComp, URootMotionModifier* Modifier) const;

	UFUNCTION(BlueprintImplementableEvent, Category = "Motion Warping")
	UE_API void OnWarpEnd(UMotionWarpingComponent* MotionWarpingComp, URootMotionModifier* Modifier) const;

#if WITH_EDITOR
	UE_API virtual void ValidateAssociatedAssets() override;

	UE_API virtual void DrawInEditor(class FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* MeshComp, const UAnimSequenceBase* Animation, const FAnimNotifyEvent& NotifyEvent) const override;
	UE_API virtual void DrawCanvasInEditor(FCanvas& Canvas, FSceneView& View, USkeletalMeshComponent* MeshComp, const UAnimSequenceBase* Animation, const FAnimNotifyEvent& NotifyEvent) const override;
#endif
};

#undef UE_API
