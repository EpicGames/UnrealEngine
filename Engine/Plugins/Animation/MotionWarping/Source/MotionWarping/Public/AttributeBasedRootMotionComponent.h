// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/ActorComponent.h"
#include "AttributeBasedRootMotionComponent.generated.h"

#define UE_API MOTIONWARPING_API

class ACharacter;

USTRUCT()
struct FAttributeBasedRootMotionComponentPrePhysicsTickFunction : public FTickFunction
{
	GENERATED_USTRUCT_BODY()

	/** CharacterMovementComponent that is the target of this tick **/
	class UAttributeBasedRootMotionComponent* Target;

	/**
	 * Abstract function actually execute the tick.
	 * @param DeltaTime - frame time to advance, in seconds
	 * @param TickType - kind of tick for this frame
	 * @param CurrentThread - thread we are executing on, useful to pass along as new tasks are created
	 * @param MyCompletionGraphEvent - completion event for this task. Useful for holding the completion of this task until certain child tasks are complete.
	 **/
	virtual void ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;

	/** Abstract function to describe this tick. Used to print messages about illegal cycles in the dependency graph **/
	virtual FString DiagnosticMessage() override;
	
	/** Function used to describe this tick for active tick reporting. **/
	virtual FName DiagnosticContext(bool bDetailed) override;
};
template<>
struct TStructOpsTypeTraits<FAttributeBasedRootMotionComponentPrePhysicsTickFunction> : public TStructOpsTypeTraitsBase2<FAttributeBasedRootMotionComponentPrePhysicsTickFunction>
{
	enum
	{
		WithCopy = false
	};
};


UENUM()
enum class EAttributeBasedRootMotionMode
{
	ApplyDelta,
	ApplyVelocity,
};



UCLASS(MinimalAPI, ClassGroup = Movement, meta = (BlueprintSpawnableComponent))
class UAttributeBasedRootMotionComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	UE_API UAttributeBasedRootMotionComponent(const FObjectInitializer& ObjectInitializer);

	UE_API virtual void InitializeComponent() override;
	UE_API virtual void RegisterComponentTickFunctions(bool bRegister) override;

	UE_API void PrePhysicsTickComponent(float DeltaTime);
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	
	/** Gets the character this component belongs to */
	inline ACharacter* GetCharacterOwner() const { return CharacterOwner.Get(); }

	UPROPERTY(BlueprintReadWrite, Category = "Runtime")
	bool bEnableRootMotion = true;
	
	
	UPROPERTY(EditAnywhere, Category = "Settings")
	EAttributeBasedRootMotionMode Mode = EAttributeBasedRootMotionMode::ApplyDelta;
	
protected:

	/** Pre-physics tick function for this character */
	FAttributeBasedRootMotionComponentPrePhysicsTickFunction PrePhysicsTickFunction;

	/** Character this component belongs to */
	UPROPERTY(Transient)
	TWeakObjectPtr<ACharacter> CharacterOwner;

	FTransform PrevTransform;
	bool PrevTransformValid = false;

	FVector TranslationVelocity {0, 0, 0};
	FVector RotationVelocity {0, 0, 0};
};

#undef UE_API
