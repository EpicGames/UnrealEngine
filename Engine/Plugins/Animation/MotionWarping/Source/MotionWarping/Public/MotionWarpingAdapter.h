// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "RootMotionModifier.h"
#include "MotionWarpingAdapter.generated.h"

DECLARE_DELEGATE_RetVal_ThreeParams(FTransform, FOnWarpLocalspaceRootMotionWithContext, const FTransform&, float, const FMotionWarpingUpdateContext*)
DECLARE_DELEGATE_RetVal_ThreeParams(FTransform, FOnWarpWorldspaceRootMotionWithContext, const FTransform&, float, const FMotionWarpingUpdateContext*)

/**
 * MotionWarpingBaseAdapter: base class to adapt/apply motion warping to a target. Concrete subclasses should override
 */
UCLASS(MinimalAPI, Abstract)
class UMotionWarpingBaseAdapter : public UObject
{
	GENERATED_BODY()

public:
	virtual ~UMotionWarpingBaseAdapter() = default;

	virtual AActor* GetActor() const { return nullptr; }
	virtual USkeletalMeshComponent* GetMesh() const { return nullptr; }

	// Current Transform. Warping should use this as child adapters may not have an actor.
	virtual FTransform GetCurrentTransform() const 
	{ 
		if (AActor* OwningActor = GetActor())
		{
			return OwningActor->GetActorTransform();
		}

		return FTransform::Identity; 
	}

	// Current visual root (IE: Bottom of actor capsule).
	virtual FVector GetVisualRootLocation() const 
	{ 
		return FVector::ZeroVector; 
	}

	// Current Transform Rotation. Utility method to make API consistent.
	FQuat GetCurrentRotation() const 
	{
		return GetCurrentTransform().GetRotation();
	}

	virtual FVector GetBaseVisualTranslationOffset() const { return FVector::ZeroVector; }
	virtual FQuat GetBaseVisualRotationOffset() const { return FQuat::Identity; }

	// A MotionWarpingComponent will bind to this delegate to perform warping when it is triggered
	FOnWarpLocalspaceRootMotionWithContext WarpLocalRootMotionDelegate;
};

