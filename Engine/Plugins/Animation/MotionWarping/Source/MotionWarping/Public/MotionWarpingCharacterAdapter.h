// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MotionWarpingAdapter.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "MotionWarpingCharacterAdapter.generated.h"

#define UE_API MOTIONWARPING_API

// Adapter for Character / ChararacterMovementComponent actors to participate in motion warping
UCLASS(MinimalAPI)
class UMotionWarpingCharacterAdapter : public UMotionWarpingBaseAdapter
{
	GENERATED_BODY()

public:
	UE_API virtual void BeginDestroy() override;

	UE_API void SetCharacter(ACharacter* InCharacter);

	UE_API virtual AActor* GetActor() const override;
	UE_API virtual USkeletalMeshComponent* GetMesh() const override;
	UE_API virtual FVector GetVisualRootLocation() const override;
	UE_API virtual FVector GetBaseVisualTranslationOffset() const override;
	UE_API virtual FQuat GetBaseVisualRotationOffset() const override;

private:
	// Triggered when the character says it's time to pre-process local root motion. This adapter catches the request and passes along to the Warping component
	FTransform WarpLocalRootMotionOnCharacter(const FTransform& LocalRootMotionTransform, UCharacterMovementComponent* TargetMoveComp, float DeltaSeconds);

	/** The associated character */
	TWeakObjectPtr<ACharacter> TargetCharacter;
};

#undef UE_API
