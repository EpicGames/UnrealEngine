// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "RootMotionModifier.h"
#include "MotionWarpingComponent.generated.h"

#define UE_API MOTIONWARPING_API

class UMotionWarpingSwitchOffCondition;
struct FSwitchOffConditionData;
template <class T> class TAutoConsoleVariable;

class ACharacter;
class UAnimInstance;
class UAnimSequenceBase;
class UCharacterMovementComponent;
class UMotionWarpingBaseAdapter;
class UMotionWarpingComponent;
class UAnimNotifyState_MotionWarping;
struct FAnimMontageInstance;
struct FCompactPose;
template<class PoseType> struct FCSPose;

DECLARE_LOG_CATEGORY_EXTERN(LogMotionWarping, Log, All);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
struct FMotionWarpingCVars
{
	static UE_API TAutoConsoleVariable<int32> CVarMotionWarpingDisable;
	static UE_API TAutoConsoleVariable<int32> CVarMotionWarpingDebug;
	static UE_API TAutoConsoleVariable<float> CVarMotionWarpingDrawDebugDuration;
	static UE_API TAutoConsoleVariable<int32> CVarWarpedTargetDebug;
	static UE_API TAutoConsoleVariable<int32> CVarWarpedSwitchOffConditionDebug;
};
#endif

USTRUCT(BlueprintType)
struct FMotionWarpingWindowData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	TObjectPtr<UAnimNotifyState_MotionWarping> AnimNotify = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	float StartTime = 0.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	float EndTime = 0.f;
};


UCLASS(MinimalAPI)
class UMotionWarpingUtilities : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Extract bone pose in local space for all bones in BoneContainer. If Animation is a Montage the pose is extracted from the first track */
	static UE_API void ExtractLocalSpacePose(const UAnimSequenceBase* Animation, const FBoneContainer& BoneContainer, float Time, bool bExtractRootMotion, FCompactPose& OutPose);

	/** Extract bone pose in component space for all bones in BoneContainer. If Animation is a Montage the pose is extracted from the first track */
	static UE_API void ExtractComponentSpacePose(const UAnimSequenceBase* Animation, const FBoneContainer& BoneContainer, float Time, bool bExtractRootMotion, FCSPose<FCompactPose>& OutPose);

	/** Extract Root Motion transform from a contiguous position range */
	UFUNCTION(BlueprintCallable, Category = "Motion Warping")
	static UE_API FTransform ExtractRootMotionFromAnimation(const UAnimSequenceBase* Animation, float StartTime, float EndTime);

	/** Extract root bone transform at a given time */
	static UE_API FTransform ExtractRootTransformFromAnimation(const UAnimSequenceBase* Animation, float Time);

	/** @return All the MotionWarping windows within the supplied animation */
	UFUNCTION(BlueprintCallable, Category = "Motion Warping")
	static UE_API void GetMotionWarpingWindowsFromAnimation(const UAnimSequenceBase* Animation, TArray<FMotionWarpingWindowData>& OutWindows);

	/** @return All the MotionWarping windows within the supplied animation for a given Warp Target */
	UFUNCTION(BlueprintCallable, Category = "Motion Warping")
	static UE_API void GetMotionWarpingWindowsForWarpTargetFromAnimation(const UAnimSequenceBase* Animation, FName WarpTargetName, TArray<FMotionWarpingWindowData>& OutWindows);

	/** @return root transform relative to the warp point bone at the supplied time */
	static UE_API FTransform CalculateRootTransformRelativeToWarpPointAtTime(const ACharacter& Character, const UAnimSequenceBase* Animation, float Time, const FName& WarpPointBoneName);
	static UE_API FTransform CalculateRootTransformRelativeToWarpPointAtTime(const UMotionWarpingBaseAdapter& WarpingAdapter, const UAnimSequenceBase* Animation, float Time, const FName& WarpPointBoneName);

	/** @return root transform relative to the warp point transform at the supplied time */
	static UE_API FTransform CalculateRootTransformRelativeToWarpPointAtTime(const ACharacter& Character, const UAnimSequenceBase* Animation, float Time, const FTransform& WarpPointTransform);
	static UE_API FTransform CalculateRootTransformRelativeToWarpPointAtTime(const UMotionWarpingBaseAdapter& WarpingAdapter, const UAnimSequenceBase* Animation, float Time, const FTransform& WarpPointTransform);

	/** Extract bone transform from animation at a given time */
	UFUNCTION(BlueprintCallable, Category = "Motion Warping")
	static UE_API void ExtractBoneTransformFromAnimationAtTime(const UAnimInstance* AnimInstance, const UAnimSequenceBase* Animation, float Time, bool bExtractRootMotion, FName BoneName, bool bLocalSpace, FTransform& OutTransform);
};


DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FMotionWarpingPreUpdate, class UMotionWarpingComponent*, MotionWarpingComp);

UCLASS(MinimalAPI, ClassGroup = Movement, meta = (BlueprintSpawnableComponent))
class UMotionWarpingComponent : public UActorComponent
{
	GENERATED_BODY()

public:

	friend class UMotionWarpingMontageTrajectoryAdapter;

public:

	/** Whether to look inside animations within montage when looking for warping windows */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Config")
	bool bSearchForWindowsInAnimsWithinMontages;

	/** Event called before Root Motion Modifiers are updated */
	UPROPERTY(BlueprintAssignable, Category = "Motion Warping")
	FMotionWarpingPreUpdate OnPreUpdate;

	UE_API UMotionWarpingComponent(const FObjectInitializer& ObjectInitializer);

	UE_API virtual void InitializeComponent() override;
	UE_API virtual void GetLifetimeReplicatedProps(TArray< FLifetimeProperty >& OutLifetimeProps) const;

	/** Set a new adapter of a particular type */
	UE_API UMotionWarpingBaseAdapter* CreateOwnerAdapter(TSubclassOf<UMotionWarpingBaseAdapter> AdapterClass);

	template <class T>
	T* CreateOwnerAdapter()
	{
		static_assert(TPointerIsConvertibleFromTo<T, const UMotionWarpingBaseAdapter>::Value, "'T' template parameter to CreateOwnerAdapter must be derived from UMotionWarpingBaseAdapter");
		return CastChecked<T>(CreateOwnerAdapter(T::StaticClass()));
	}

	/** Get the current adapter to the owner */
	UMotionWarpingBaseAdapter* GetOwnerAdapter() const { return OwnerAdapter; }

	/** Returns the list of root motion modifiers */
	inline const TArray<URootMotionModifier*>& GetModifiers() const { return Modifiers; }

	/** Check if we contain a RootMotionModifier for the supplied animation and time range */
	UE_API bool ContainsModifier(const UAnimSequenceBase* Animation, float StartTime, float EndTime) const;

	/** Add a new modifier */
	UE_API int32 AddModifier(URootMotionModifier* Modifier);

	/** Mark all the modifiers as Disable */
	UFUNCTION(BlueprintCallable, Category = "Motion Warping")
	UE_API void DisableAllRootMotionModifiers();

	UE_API URootMotionModifier* AddModifierFromTemplate(URootMotionModifier* Template, const UAnimSequenceBase* Animation, float StartTime, float EndTime);

	/** Find the target associated with a specified name */
	inline const FMotionWarpingTarget* FindWarpTarget(const FName& WarpTargetName) const
	{
		return WarpTargets.FindByPredicate([&WarpTargetName](const FMotionWarpingTarget& WarpTarget){ return WarpTarget.Name == WarpTargetName; });
	}

	UE_EXPERIMENTAL(5.8, "This API is experimental and might be removed without warning, not for production use.")
	UFUNCTION(BlueprintCallable, Category = "Motion Warping")
	UE_API void FindWarpTarget(FName WarpTargetName, FMotionWarpingTarget& MotionWarpingTarget, bool& bFound)
	{
		if (const FMotionWarpingTarget* FoundMotionWarpingTarget = FindWarpTarget(WarpTargetName))
		{
			MotionWarpingTarget = *FoundMotionWarpingTarget;
			bFound = true;
		}
		else
		{
			MotionWarpingTarget = FMotionWarpingTarget();
			bFound = false;
		}
	}

	/** Returns the full list of warp targets */
	inline TConstArrayView<FMotionWarpingTarget> GetWarpTargets() const { return WarpTargets; }

	/** Adds or update a warp target */
	UFUNCTION(BlueprintCallable, Category = "Motion Warping")
	UE_API void AddOrUpdateWarpTarget(const FMotionWarpingTarget& WarpTarget);

	/** 
	 * Create and adds or update a target associated with a specified name 
	 * @param WarpTargetName Warp target identifier
	 * @param TargetTransform Transform used to set the location and rotation for the warp target
	 */
	UFUNCTION(BlueprintCallable, Category = "Motion Warping")
	UE_API void AddOrUpdateWarpTargetFromTransform(FName WarpTargetName, FTransform TargetTransform);

	/**
	 * Create and adds or update a target associated with a specified name
	 * @param WarpTargetName Warp target identifier
	 * @param Component Scene Component used to get the target transform
	 * @param BoneName Optional bone or socket in the component used to get the target transform. 
	 * @param bFollowComponent Whether the target transform should update while the warping is active. Useful for tracking moving targets.
	 *		  Note that this will be one frame off if our owner ticks before the target actor. Add tick pre-requisites to avoid this.
	 * @param LocationOffsetDirection Direction of LocationOffset
	 * @param LocationOffset Optional translation offset to apply to the transform we get from the component
	 * @param RotationOffset Optional rotation offset to apply to the transform we get from the component
	 */
	UFUNCTION(BlueprintCallable, Category = "Motion Warping")
	UE_API void AddOrUpdateWarpTargetFromComponent(FName WarpTargetName, const USceneComponent* Component, FName BoneName, bool bFollowComponent, EWarpTargetLocationOffsetDirection LocationOffsetDirection = EWarpTargetLocationOffsetDirection::TargetsForwardVector, FVector LocationOffset = FVector::ZeroVector, FRotator RotationOffset = FRotator::ZeroRotator);
	
	// @TODO: Deprecate this.
	UE_API void AddOrUpdateWarpTargetFromComponent(FName WarpTargetName, const USceneComponent* Component, FName BoneName, bool bFollowComponent, FVector LocationOffset = FVector::ZeroVector, FRotator RotationOffset = FRotator::ZeroRotator);

	/**
	 * Create and adds or update a target associated with a specified name
	 * @param WarpTargetName Warp target identifier
	 * @param TargetLocation Location for the warp target
	 */
	UFUNCTION(BlueprintCallable, Category = "Motion Warping")
	void AddOrUpdateWarpTargetFromLocation(FName WarpTargetName, FVector TargetLocation)
	{
		AddOrUpdateWarpTargetFromTransform(WarpTargetName, FTransform(TargetLocation));
	}

	/**
	 * Create and adds or update a target associated with a specified name
	 * @param WarpTargetName Warp target identifier
	 * @param TargetLocation Location for the warp target
	 */
	UFUNCTION(BlueprintCallable, Category = "Motion Warping")
	void AddOrUpdateWarpTargetFromLocationAndRotation(FName WarpTargetName, FVector TargetLocation, FRotator TargetRotation)
	{
		AddOrUpdateWarpTargetFromTransform(WarpTargetName, FTransform(TargetRotation, TargetLocation));
	}

	/** Removes the warp target associated with the specified key  */
	UFUNCTION(BlueprintCallable, Category = "Motion Warping")
	UE_API int32 RemoveWarpTarget(FName WarpTargetName);

	/** Removes all warp targets */
	UFUNCTION(BlueprintCallable, Category = "Motion Warping")
	UE_API int32 RemoveAllWarpTargets();

	/** Removes multiple warp targets */
	UFUNCTION(BlueprintCallable, Category = "Motion Warping")
	UE_API int32 RemoveWarpTargets(const TArray<FName>& WarpTargetNames);

	UFUNCTION(BlueprintCallable, Category="Motion Warping|Experimental")
	UE_API void AddSwitchOffCondition(FName WarpTargetName, UMotionWarpingSwitchOffCondition* Condition);
	
	UE_API void RemoveSwitchOffConditions(FName WarpTargetName);
	
	UE_API FSwitchOffConditionData* FindSwitchOffConditionData(FName WarpTargetName);

	/** Creates Prediction adapter if needed and swaps state to prediction state */
	UE_API void BeginTrajectoryPredictions(const FAnimMontageInstance* MontageInstance);

	/** Reverts state back to before any predictions */
	UE_API void EndTrajectoryPredictions();

	/** True if we are currentl prediciting a trajectory */
	UE_API bool IsPredictingTrajectory() const;

protected:

	/** Adapter that connects motion warping to an owner */
	UPROPERTY(Transient, VisibleInstanceOnly, Category = "Warp State")
	TObjectPtr<UMotionWarpingBaseAdapter> OwnerAdapter;

	/** Experimental. Adapter used to predict trajectories. */
	UE_EXPERIMENTAL(5.8, "Prediction Warping API subject to change.")
	UPROPERTY(Transient, VisibleInstanceOnly, Category = "Warp State")
	TObjectPtr<UMotionWarpingBaseAdapter> PredictionAdapter;

	/** Experimental. Cache of owner adapter before swapped out to do prediction, swapped back post prediction */
	UE_EXPERIMENTAL(5.8, "Prediction Warping API subject to change.")
	UPROPERTY(Transient, VisibleInstanceOnly, Category = "Warp State")
	TObjectPtr<UMotionWarpingBaseAdapter> CachedOwnerAdapter;
	
	/** List of root motion modifiers */
	UPROPERTY(Transient, VisibleInstanceOnly, Category = "Warp State")
	TArray<TObjectPtr<URootMotionModifier>> Modifiers;

	UPROPERTY(Transient, Replicated, VisibleInstanceOnly, Category = "Warp State")
	TArray<FMotionWarpingTarget> WarpTargets;
		
	UPROPERTY(Transient, Experimental, VisibleInstanceOnly, Category = "Warp State")
    TArray<FSwitchOffConditionData> SwitchOffConditions;

	/** True if we are predicting trajectory via a temp swap state. */
	UE_EXPERIMENTAL(5.8, "Prediction Warping API subject to change.")
	UPROPERTY(Transient, Experimental, VisibleInstanceOnly, Category = "Warp State")
	bool bPredictingTrajectory = false;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	TOptional<FVector> OriginalRootMotionAccum;
	TOptional<FVector> WarpedRootMotionAccum;
#endif

	UE_API void UpdateSwitchOffConditions();

	UE_API void UpdateWithContext(const FMotionWarpingUpdateContext& Context, float DeltaSeconds);

	UE_API bool FindAndUpdateWarpTarget(const FMotionWarpingTarget& WarpTarget);

	UE_API FTransform ProcessRootMotionPreConvertToWorld(const FTransform& InRootMotion, float DeltaSeconds, const FMotionWarpingUpdateContext* InContext=nullptr); // callback with optional context
};

#undef UE_API
