// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "MotionWarpingFunctionLibrary.generated.h"

#define UE_API MOTIONWARPING_API

/**
 * Motion Warping Function Library
 */
UCLASS(MinimalAPI, meta = (ScriptName = "MotionWarpingLibrary"))
class UMotionWarpingFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintPure, Category = "Motion Warping", meta = (NativeMakeFunc))
	static UE_API FMotionWarpingTarget MakeMotionWarpingTarget(const FName Name, const FVector Location, const FRotator Rotation, const USceneComponent* Component, FName BoneName, bool bFollowComponent, EWarpTargetLocationOffsetDirection LocationOffsetDirection, const AActor* AvatarActor, const FVector LocationOffset, const FRotator RotationOffset);

};

#undef UE_API
