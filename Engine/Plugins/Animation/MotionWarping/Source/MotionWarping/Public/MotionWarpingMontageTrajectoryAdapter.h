// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MotionWarpingAdapter.h"
#include "MotionWarpingMontageTrajectoryAdapter.generated.h"

#define UE_API MOTIONWARPING_API

class UMotionWarpingSwitchOffCondition;
struct FAnimMontageInstance;
struct FSwitchOffConditionData;

/** Temp state to iterate on for properties normally owned by the warping component. Used so we can predict without mutating real actor state. */
USTRUCT(Experimental)
struct FMotionWarpingSwapState
{
	GENERATED_BODY()

	/** List of root motion modifiers, stored as modifiers can be removed during prediction */
	UPROPERTY(Transient, VisibleInstanceOnly, Category = "Warp State")
	TArray<TObjectPtr<URootMotionModifier>> Modifiers;

	/** List of warp targets */
	UPROPERTY(Transient, VisibleInstanceOnly, Category = "Warp State")
	TArray<FMotionWarpingTarget> WarpTargets;

	/** 
	 * List of switch off conditions
	 * 
	 * @TODO: Currently unused. Switch off conditions aren't actorless yet & to be accurate we may want to also predict against target actor trajectories
	 */
	UPROPERTY(Transient, Experimental, VisibleInstanceOnly, Category = "Warp State")
	TArray<FSwitchOffConditionData> SwitchOffConditions;

	/** State of modifiers prior to prediction */
	TArray<TSharedPtr<FRootMotionModifierSwapState>> ModifierSwapStates;
};

/** Adapter for montage trajectories to participate in motion warping */
UCLASS(Experimental, MinimalAPI)
class UMotionWarpingMontageTrajectoryAdapter : public UMotionWarpingBaseAdapter
{
	GENERATED_BODY()

public:

	/** Initialize current transforms from some other adapter & swaps component state to prediction state. */
	UE_API void InitializeFrom(const UMotionWarpingBaseAdapter* InAdapter, const FAnimMontageInstance* InMontageInstance);

	/** Restores component state to before any prediction has been done */
	UE_API void RestoreState();

	UE_API virtual FTransform GetCurrentTransform() const override;
	UE_API virtual FVector GetVisualRootLocation() const override;

	UE_API void SetCurrentTransform(const FTransform InCurrentTransform);
	UE_API void SetVisualRootLocation(const FVector InVisualRootLocation);

	UE_API virtual FVector GetBaseVisualTranslationOffset() const override;
	UE_API virtual FQuat GetBaseVisualRotationOffset() const override;

	/** Triggered iteratively as we sample the trajectory */
	UE_API FTransform WarpLocalRootMotionOnMontageTrajectory(const FTransform& LocalRootMotionTransform, float DeltaSeconds, float InMontagePreviousPosition, float InMontageCurrentPosition);

public:

	/** Cached state before any prediction, we restore this after predicting. */
	UPROPERTY(Transient)
	FMotionWarpingSwapState CachedWarpState;

	/** Montage to warp trajectory for */
	UPROPERTY(Transient)
	TObjectPtr<const UAnimMontage> Montage;

private:

	/** Warping properties captured in adapter initialization that update as we iterate*/
	FTransform CurrentTransform = FTransform::Identity;
	FVector VisualRootLocation = FVector::ZeroVector;

	/** Warping properties that do not change after adapter initialization */ 
	FVector BaseVisualTranslationOffset = FVector::ZeroVector;
	FQuat BaseVisualRotationOffset = FQuat::Identity;

	/** Properties for warp context state */
	float MontageWeight = 0.0f;
	float MontagePlayRate = 0.0f;
};

#undef UE_API
