// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RootMotionModifier.h"
#include "MotionWarpingSwitchOffCondition.generated.h"

#define UE_API MOTIONWARPING_API

class UMotionWarpingSwitchOffCondition;
struct FMotionWarpingTarget;

USTRUCT(BlueprintType, Experimental)
struct FSwitchOffConditionData
{
	GENERATED_BODY()

	FSwitchOffConditionData()
		: WarpTargetName(NAME_None), SwitchOffConditions(TArray<UMotionWarpingSwitchOffCondition*>())
	{}
	
	FSwitchOffConditionData(FName InWarpTargetName, UMotionWarpingSwitchOffCondition* InSwitchOffCondition)
		: WarpTargetName(InWarpTargetName)
	{
		SwitchOffConditions.Add(InSwitchOffCondition);
	}

	FSwitchOffConditionData(FName InWarpTargetName)
		: WarpTargetName(InWarpTargetName), SwitchOffConditions(TArray<UMotionWarpingSwitchOffCondition*>())
	{}
	
	UPROPERTY(BlueprintReadWrite, Category = "Defaults")
	FName WarpTargetName;

	UPROPERTY(BlueprintReadWrite, Category = "Defaults")
	TArray<TObjectPtr<UMotionWarpingSwitchOffCondition>> SwitchOffConditions;

	UE_API void SetMotionWarpingTarget(const FMotionWarpingTarget* MotionWarpingTarget);
};

/**Result of switch off condition.*/
UENUM(BlueprintType, Category = "MotionWarping", meta = (Experimental))
enum class ESwitchOffConditionEffect : uint8
{
	/**Changes associated motion warping target from component to a location of this component
	* in the frame in which this switch off condition appeared */
	CancelFollow UMETA(DisplayName = "Cancel Follow Component"),

	/**Removes associated motion warping target*/
	CancelWarping UMETA(DisplayName = "Cancel Warping"),

	/**During time slot in the animation, where switch off condition is true, only play root motion, without warping*/
	PauseWarping UMETA(DisplayName = "Pause Warping"),

	/**During time slot in the animation, where switch off condition is true, play the animation in place*/
	PauseRootMotion UMETA(DisplayName = "Pause Root Motion")
};

UCLASS(MinimalAPI, Abstract, EditInlineNew, Experimental, ClassGroup = MotionWarping)
class UMotionWarpingSwitchOffCondition : public UObject
{
	GENERATED_BODY()

public:
	/** If set to false, switch off condition will use target actor location */
	UPROPERTY(EditAnywhere, Category = "MotionWarping")
	bool bUseWarpTargetAsTargetLocation = true;

	UPROPERTY(EditAnywhere, Category = "MotionWarping")
	ESwitchOffConditionEffect Effect = ESwitchOffConditionEffect::CancelWarping;
	
	/**
	 *	Initialize switch off condition
	 *	@param InOwnerActor Actor with MotionWarpingComponent, owning corresponding WarpingTarget
	 *	@param InTargetActor Target actor used for calculating switch off condition result
	 */
	ESwitchOffConditionEffect GetEffect() const { return Effect; }
	UE_API bool Check() const;
	
	/**
	 * If bUseWarpTargetAsTargetLocation is true, this will return target FMotionWarpingTarget location.
	 * If bUseWarpTargetAsTargetLocation is false, this will return target actor location.
	 */
	UE_API FVector GetTargetLocation() const;

	/**
	* If bUseWarpTargetAsTargetLocation is true, this will return target FMotionWarpingTarget rotation.
	* If bUseWarpTargetAsTargetLocation is false, this will return target actor rotation.
	*/
	UE_API FRotator GetTargetRotation() const;

	/**
	 * Set warp target as context for calculating switch off condition result
	 * if bUseWarpTargetAsTargetLocation is set to true
	 */
	UE_API virtual void SetWarpTargetForDestination(const FMotionWarpingTarget* InMotionWarpingTarget);
	virtual bool OnCheck() const { return false; }

	/**
	 * Extra information used for debugging, if CVAR MotionWarping.Debug.SwitchOffCondition is true
	 */
	virtual FString ExtraDebugInfo() const { return FString(); }
	UE_API virtual bool IsConditionValid() const;

	virtual void SetOwnerActor(const AActor* InOwnerActor) { OwnerActor = InOwnerActor; }
	virtual void SetTargetActor(const AActor* InTargetActor) { TargetActor = InTargetActor; }
	virtual void SetMotionWarpingTarget(const FMotionWarpingTarget* InMotionWarpingTarget) { MotionWarpingTarget = InMotionWarpingTarget; }
	
protected:
	bool bIsInitialized = false;

	UPROPERTY()
	TObjectPtr<const AActor> OwnerActor;

	UPROPERTY()
	TObjectPtr<const AActor> TargetActor;

	const FMotionWarpingTarget* MotionWarpingTarget = nullptr;
};

UENUM(BlueprintType, Category = "MotionWarping", meta = (Experimental))
enum class ESwitchOffConditionDistanceOp : uint8
{
	LessThan,
	GreaterThan
};

UENUM(BlueprintType, Category = "MotionWarping", meta = (Experimental))
enum class ESwitchOffConditionDistanceAxesType : uint8
{
	AllAxes,
	IgnoreZAxis,
	OnlyZAxis,
};

UCLASS(Experimental)
class UMotionWarpingSwitchOffDistanceCondition : public UMotionWarpingSwitchOffCondition
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "MotionWarping")
	ESwitchOffConditionDistanceOp Operator;

	UPROPERTY(EditAnywhere, Category = "MotionWarping")
	ESwitchOffConditionDistanceAxesType AxesType;
	
	UPROPERTY(EditAnywhere, Category = "MotionWarping")
	float Distance;

	/**
	 * Creates Switch Off Distance Condition that checks distance between Owner Actor and Target Location.
	 * If Use Warp Target Location is true, Target Location is corresponding Warp Target's location.
	 * If Use Warp Target Location is false, Target Location is Target Actor's parameter location.
	 * @param InOwnerActor Actor owning Motion Warping Component to which this switch off condition will be added.
	 * @param InEffect What should happen if switch off condition is true
	 * @param InOperator Distance comparison operator
	 * @param InAxesType Specifier of the axes the distance should be calculated for
	 * @param InDistance Distance to compare to
	 * @param InbUseWarpTargetAsTargetLocation Should use corresponding warp target as context for this Switch Off Condition.
	 * If set to false, will use TargetActor parameter
	 * @param InTargetActor Target Actor to use as a context for this Switch Off Condition if Use Warp Target As Location parameter is false.
	 * @return Created Switch Off Condition. This Switch Off Condition can be added to Motion Warping Component with Add Switch Off Condition node.
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Warping", meta=(AdvancedDisplay="bInUseWarpTargetAsTargetLocation, InTargetActor"))
	static UMotionWarpingSwitchOffDistanceCondition* CreateSwitchOffDistanceCondition(
		UPARAM(DisplayName = "Owner Actor", ref)		AActor* InOwnerActor,
		UPARAM(DisplayName = "Effect")					ESwitchOffConditionEffect InEffect,
		UPARAM(DisplayName = "Operator")				ESwitchOffConditionDistanceOp InOperator,
		UPARAM(DisplayName = "AxesType")				ESwitchOffConditionDistanceAxesType InAxesType,
		UPARAM(DisplayName = "Distance")				float InDistance,
		UPARAM(DisplayName = "UseWarpTargetAsLocation") bool InbUseWarpTargetAsTargetLocation = true,
		UPARAM(DisplayName = "TargetActor")				AActor* InTargetActor = nullptr);
	
protected:
	virtual bool OnCheck() const override;
	virtual FString ExtraDebugInfo() const override;
	
	float CalculateSqDistance() const;
	float CalculateSqDistance2D() const;
	float CalculateZDistance() const;
};


UENUM(BlueprintType, Category = "MotionWarping", meta = (Experimental))
enum class ESwitchOffConditionAngleOp : uint8
{
	LessThan,
	GreaterThan
};

UCLASS(Experimental)
class UMotionWarpingSwitchOffAngleToTargetCondition : public UMotionWarpingSwitchOffCondition
{
	GENERATED_BODY()

public:
	/**
	 * Creates Switch Off Angle To Target Condition that checks angle between Owner Actor and Target Location.
	 * If Use Warp Target Location is true, Target Location is corresponding Warp Target's location.
	 * If Use Warp Target Location is false, Target Location is Target Actor's parameter location.
	 * @param InOwnerActor Actor owning Motion Warping Component to which this switch off condition will be added.
	 * @param InEffect What should happen if switch off condition is true
	 * @param InOperator Angle comparison operator
	 * @param InAngle Angle to compare to
	 * @param bInIgnoreZAxis Should ignore Z axis in Angle comparison
	 * @param bInUseWarpTargetAsTargetLocation Should use corresponding warp target as context for this Switch Off Condition.
	 * If set to false, will use TargetActor parameter
	 * @param InTargetActor Target Actor to use as a context for this Switch Off Condition if Use Warp Target As Location parameter is false.
	 * @return Created Switch Off Condition. This Switch Off Condition can be added to Motion Warping Component with Add Switch Off Condition node.
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Warping|Experimental", meta=(AdvancedDisplay="bInUseWarpTargetAsTargetLocation, InTargetActor"))
	static UMotionWarpingSwitchOffAngleToTargetCondition* CreateSwitchOffAngleToTargetCondition(
		UPARAM(DisplayName = "Owner Actor", ref)		AActor* InOwnerActor,
		UPARAM(DisplayName = "Effect")					ESwitchOffConditionEffect InEffect,
		UPARAM(DisplayName = "Operator")				ESwitchOffConditionAngleOp InOperator,
		UPARAM(DisplayName = "Angle")					float InAngle,
		UPARAM(DisplayName = "Ignore Z Axis")			bool bInIgnoreZAxis,
		UPARAM(DisplayName = "TargetActor")				bool bInUseWarpTargetAsTargetLocation = true,
		UPARAM(DisplayName = "TargetActor")				AActor* InTargetActor = nullptr);
	
protected:
	virtual bool OnCheck() const override;
	virtual FString ExtraDebugInfo() const override;

	float CalculateAngleToTarget() const;

	UPROPERTY(EditAnywhere, Category = "MotionWarping")
	ESwitchOffConditionAngleOp Operator;

	UPROPERTY(EditAnywhere, Category = "MotionWarping", meta = (ForceUnits="degrees"))
	float Angle;

	UPROPERTY(EditAnywhere, Category = "MotionWarping")
	bool bIgnoreZAxis;
};

UENUM(BlueprintType, Category = "MotionWarping", meta = (Experimental))
enum class ESwitchOffConditionCompositeOp : uint8
{
	Or UMETA(DisplayName = "OR"),
	And UMETA(DisplayName = "AND")
};

UCLASS(Experimental)
class UMotionWarpingSwitchOffCompositeCondition : public UMotionWarpingSwitchOffCondition
{
	GENERATED_BODY()
public:
	virtual void SetOwnerActor(const AActor* InOwnerActor) override;
	virtual void SetTargetActor(const AActor* InTargetActor) override;
	virtual void SetMotionWarpingTarget(const FMotionWarpingTarget* InMotionWarpingTarget) override;

	/**
	 * Creates Switch Off Composite Condition that lets you combine different switch off conditions with logic AND/OR operators.
	 * @param InOwnerActor Actor owning Motion Warping Component to which this switch off condition will be added.
	 * @param InEffect What should happen if switch off condition is true
	 * @param InSwitchOffConditionA First Switch Off Condition to combine
	 * @param InLogicOperator Logic operator to use for Switch Off Condition combination
	 * @param InSwitchOffConditionB Second Switch Off Condition to combine
	 * @param bInUseWarpTargetAsTargetLocation Should use corresponding warp target as context for this Switch Off Condition.
	 * If set to false, will use TargetActor parameter
	 * @param InTargetActor Target Actor to use as a context for this Switch Off Condition if Use Warp Target As Location parameter is false.
	 * @return Created Switch Off Condition. This Switch Off Condition can be added to Motion Warping Component with Add Switch Off Condition node.
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Warping|Experimental", meta=(AdvancedDisplay="bInUseWarpTargetAsTargetLocation, InTargetActor"))
	static UMotionWarpingSwitchOffCompositeCondition* CreateSwitchOffCompositeCondition(
		UPARAM(DisplayName = "Owner Actor", ref)			AActor* InOwnerActor,
		UPARAM(DisplayName = "Effect")						ESwitchOffConditionEffect InEffect,
		UPARAM(DisplayName = "Switch Off Condition A", ref)	UMotionWarpingSwitchOffCondition* InSwitchOffConditionA,
		UPARAM(DisplayName = "Logic Operator")				ESwitchOffConditionCompositeOp InLogicOperator,
		UPARAM(DisplayName = "Switch Off Condition B", ref)	UMotionWarpingSwitchOffCondition* InSwitchOffConditionB,
		UPARAM(DisplayName = "Use Warp Target As Location")	bool bInUseWarpTargetAsTargetLocation = true,
		UPARAM(DisplayName = "TargetActor")					AActor* InTargetActor = nullptr);
	
protected:
	virtual void SetWarpTargetForDestination(const FMotionWarpingTarget* InMotionWarpingTarget) override;
	virtual bool OnCheck() const override;
	virtual FString ExtraDebugInfo() const override;
	virtual bool IsConditionValid() const override;
	
	UPROPERTY(EditAnywhere, Instanced, Category="MotionWarping")
	TObjectPtr<UMotionWarpingSwitchOffCondition> SwitchOffConditionA;

	UPROPERTY(EditAnywhere, Category="MotionWarping")
	ESwitchOffConditionCompositeOp LogicOperator;

	UPROPERTY(EditAnywhere, Instanced, Category="MotionWarping")
	TObjectPtr<UMotionWarpingSwitchOffCondition> SwitchOffConditionB;
};

UCLASS(Blueprintable, Experimental)
class UMotionWarpingSwitchOffBlueprintableCondition : public UMotionWarpingSwitchOffCondition
{
	GENERATED_BODY()
	
	virtual bool OnCheck() const override;
	virtual FString ExtraDebugInfo() const override;

	/**
	 * Creates Switch Off Blueprintable Condition from WarpingSwitchOffBlueprintableCondition subclass.
	 * @param InOwnerActor Actor owning Motion Warping Component to which this switch off condition will be added.
	 * @param InEffect What should happen if switch off condition is true
	 * @param InBlueprintableCondition Subclass of WarpingSwitchOffBlueprintableCondition
	* @param bInUseWarpTargetAsTargetLocation Should use corresponding warp target as context for this Switch Off Condition.
	 * If set to false, will use TargetActor parameter 
	 * @param InTargetActor Target Actor to use as a context for this Switch Off Condition if Use Warp Target As Location parameter is false.
	 * @return Created Switch Off Condition. This Switch Off Condition can be added to Motion Warping Component with Add Switch Off Condition node.
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Warping|Experimental", meta=(AdvancedDisplay="bInUseWarpTargetAsTargetLocation, InTargetActor"))
	static UMotionWarpingSwitchOffBlueprintableCondition* CreateSwitchOffBlueprintableCondition(
		UPARAM(DisplayName = "Owner Actor", ref)					AActor* InOwnerActor,
		UPARAM(DisplayName = "Effect")								ESwitchOffConditionEffect InEffect,
		UPARAM(DisplayName = "Blueprintable Switch Off Condition")	TSubclassOf<UMotionWarpingSwitchOffBlueprintableCondition> InBlueprintableCondition,
		UPARAM(DisplayName = "Use Warp Target As Location")			bool bInUseWarpTargetAsTargetLocation = true,
		UPARAM(DisplayName = "TargetActor")							AActor* InTargetActor = nullptr);
	
public:
	virtual UWorld* GetWorld() const override;

	UFUNCTION(BlueprintNativeEvent)
	bool BP_Check(
		UPARAM(DisplayName = "Owner Actor")							const AActor* InOwnerActor,
		UPARAM(DisplayName = "Target Actor")						const AActor* InTargetActor,
		UPARAM(DisplayName = "Target Location")						FVector InTargetLocation,
		UPARAM(DisplayName = "Use warp target as target location")	bool bInUseWarpTargetAsTargetLocation) const;

	UFUNCTION(BlueprintNativeEvent)
	FString BP_ExtraDebugInfo(
		UPARAM(DisplayName = "Owner Actor")							const AActor* InOwnerActor,
		UPARAM(DisplayName = "Target Actor")						const AActor* InTargetActor,
		UPARAM(DisplayName = "Target Location")						FVector InTargetLocation,
		UPARAM(DisplayName = "Use warp target as target location")	bool bInUseWarpTargetAsTargetLocation) const;
};

#undef UE_API
