// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AlphaBlend.h"
#include "Animation/AnimSequenceBase.h"
#include "RootMotionModifier.generated.h"

#define UE_API MOTIONWARPING_API

class UMotionWarpingComponent;
class UAnimNotifyState_MotionWarping;
class URootMotionModifier;
class USceneComponent;
class ACharacter;
class UMotionWarpingBaseAdapter;

struct FRootMotionModifierSwapState;

/** 
 * Context passed to any active root motion modifier during the update phase. 
 * Contains relevant data from the animation that contributed to root motion this frame (or in the past when replaying saved moves)
 */
USTRUCT()
struct FMotionWarpingUpdateContext
{
	GENERATED_BODY()

	UPROPERTY()
	TWeakObjectPtr<const UAnimSequenceBase> Animation = nullptr;

	UPROPERTY()
	float PreviousPosition = 0.f;

	UPROPERTY()
	float CurrentPosition = 0.f;

	UPROPERTY()
	float Weight = 0.f;

	UPROPERTY()
	float PlayRate = 1.f;

	UPROPERTY()
	float DeltaSeconds = 0.f;
};

/** The possible states of a Root Motion Modifier */
UENUM(BlueprintType)
enum class ERootMotionModifierState : uint8
{
	/** The modifier is waiting for the animation to hit the warping window */
	Waiting,

	/** The modifier is active and currently affecting the final root motion */
	Active,

	/** The modifier has been marked for removal. Usually because the warping window is done */
	MarkedForRemoval,

	/** The modifier will remain in the list (as long as the window is active) but will not modify the root motion */
	Disabled
};

DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnRootMotionModifierDelegate, UMotionWarpingComponent*, MotionWarpingComp, URootMotionModifier*, RootMotionModifier);

// URootMotionModifier
///////////////////////////////////////////////////////////////

UCLASS(MinimalAPI, Abstract, BlueprintType, EditInlineNew)
class URootMotionModifier : public UObject
{
	GENERATED_BODY()

public:

	/** Source of the root motion we are warping */
	UPROPERTY(BlueprintReadOnly, Category = "Defaults")
	TWeakObjectPtr<const UAnimSequenceBase> Animation = nullptr;

	/** Start time of the warping window */
	UPROPERTY(BlueprintReadOnly, Category = "Defaults")
	float StartTime = 0.f;

	/** End time of the warping window */
	UPROPERTY(BlueprintReadOnly, Category = "Defaults")
	float EndTime = 0.f;

	/** Previous playback time of the animation */
	UPROPERTY(BlueprintReadOnly, Category = "Defaults")
	float PreviousPosition = 0.f;

	/** Current playback time of the animation */
	UPROPERTY(BlueprintReadOnly, Category = "Defaults")
	float CurrentPosition = 0.f;

	/** Current blend weight of the animation */
	UPROPERTY(BlueprintReadOnly, Category = "Defaults")
	float Weight = 0.f;

	/** Current play rate of the animation */
	UPROPERTY(BlueprintReadOnly, Category = "Defaults")
	float PlayRate = 1.f;

	/** Character owner transform at the time this modifier becomes active */
	UPROPERTY(Transient, BlueprintReadOnly, Category = "Defaults")
	FTransform StartTransform;

	/** Actual playback time when the modifier becomes active */
	UPROPERTY(Transient, BlueprintReadOnly, Category = "Defaults")
	float ActualStartTime = 0.f;

	/** Total root motion within the warping window */
	UPROPERTY(Transient, BlueprintReadOnly, Category = "Defaults")
	FTransform TotalRootMotionWithinWindow;

	/** Delegate called when this modifier is activated (starts affecting the root motion) */
	UPROPERTY()
	FOnRootMotionModifierDelegate OnActivateDelegate;

	/** Delegate called when this modifier updates while active (affecting the root motion) */
	UPROPERTY()
	FOnRootMotionModifierDelegate OnUpdateDelegate;

	/** Delegate called when this modifier is deactivated (stops affecting the root motion) */
	UPROPERTY()
	FOnRootMotionModifierDelegate OnDeactivateDelegate;

	UE_API URootMotionModifier(const FObjectInitializer& ObjectInitializer);

	/** Called when the state of the modifier changes */
	UE_API virtual void OnStateChanged(ERootMotionModifierState LastState);

	/** Sets the state of the modifier */
	UE_API void SetState(ERootMotionModifierState NewState);

	/** Returns the state of the modifier */
	inline ERootMotionModifierState GetState() const { return State; }

	/** Returns a pointer to the component that owns this modifier */
	UE_API UMotionWarpingComponent* GetOwnerComponent() const;

	/** Returns a pointer to the adapter to the object being warped */
	UE_API UMotionWarpingBaseAdapter* GetOwnerAdapter() const;

	/** Returns a pointer to the actor that owns the component that owns this modifier */
	UE_API AActor* GetActorOwner() const;
	
	/** Returns a snapshot of the current state of this modifier */
	UE_API virtual TSharedPtr<FRootMotionModifierSwapState> GetSwapState() const;

	/** Restores this modifer to some previous state snapshot */
	UE_API virtual void RestoreFromSwapState(TSharedPtr<FRootMotionModifierSwapState> InSwapState);

	/** Populates given swap state */
	UE_API virtual void PopulateSwapState(TSharedPtr<FRootMotionModifierSwapState> InSwapState) const;

	UE_API virtual void Update(const FMotionWarpingUpdateContext& Context);
	virtual FTransform ProcessRootMotion(const FTransform& InRootMotion, float DeltaSeconds) { return FTransform::Identity; }

	inline const UAnimSequenceBase* GetAnimation() const { return Animation.Get(); }

#if WITH_EDITOR	
	virtual void DrawInEditor(class FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* MeshComp, const UAnimSequenceBase* InAnimation, const FAnimNotifyEvent& NotifyEvent) const {}
	virtual void DrawCanvasInEditor(class FCanvas& Canvas, class FSceneView& View, USkeletalMeshComponent* MeshComp, const UAnimSequenceBase* InAnimation, const FAnimNotifyEvent& NotifyEvent) const {}
#endif	

private:

	friend UMotionWarpingComponent;

	/** Current state */
	UPROPERTY()
	ERootMotionModifierState State = ERootMotionModifierState::Waiting;
};

/** Stateful properties for URootMotionModifier */
struct FRootMotionModifierSwapState
{
	virtual ~FRootMotionModifierSwapState() = default;

	float StartTime = 0.f;
	float EndTime = 0.f;
	float PreviousPosition = 0.f;
	float CurrentPosition = 0.f;
	float Weight = 0.f;
	float PlayRate = 1.f;
	FTransform StartTransform;
	float ActualStartTime = 0.f;
	FTransform TotalRootMotionWithinWindow;
	ERootMotionModifierState State = ERootMotionModifierState::Waiting;
};

UENUM(BlueprintType, meta=(Experimental))
enum class EWarpTargetLocationOffsetDirection : uint8
{
	/* LocationOffset.X translates the target transform along the target component's forward vector. */
	TargetsForwardVector,
	/* LocationOffset.X translates the target transform along the vector from the target to the owning actor. */
	VectorFromTargetToOwner,
	/* LocationOffset moves the target transform in world space */
	WorldSpace
};

// URootMotionModifier_Warp
///////////////////////////////////////////////////////////////

/** Represents a point of alignment in the world */
USTRUCT(BlueprintType, meta = (HasNativeMake = "/Script/MotionWarping.MotionWarpingFunctionLibrary.MakeMotionWarpingTarget"))
struct FMotionWarpingTarget
{
	GENERATED_BODY()

	/** Unique name for this warp target */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	FName Name;

	/** When the warp target is created from a component this stores the location of the component at the time of creation, otherwise its the location supplied by the user */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	FVector Location;

	/** When the warp target is created from a component this stores the rotation of the component at the time of creation, otherwise its the rotation supplied by the user */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	FRotator Rotation;

	/** Optional component used to calculate the final target transform */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	TWeakObjectPtr<const USceneComponent> Component;

	/** Optional bone name in the component used to calculate the final target transform */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	FName BoneName;

	/** Whether the target transform calculated from a component and an optional bone should be updated during the warp 
	Only relevant when the warp target is created from a component  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	bool bFollowComponent;

	/** Determines how LocationOffset is applied by FMotionWarpingTarget when calculating a target transform
	Only relevant when the warp target is created from a component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Experimental)
	EWarpTargetLocationOffsetDirection LocationOffsetDirection = EWarpTargetLocationOffsetDirection::TargetsForwardVector;

	/** Optional static location offset. Only relevant when the warp target is created from a component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	FVector LocationOffset = FVector::ZeroVector;

	/** Optional static rotation offset. Only relevant when the warp target is created from a component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	FRotator RotationOffset = FRotator::ZeroRotator;

	/** Optional Actor that uses this warping target. Used for offset calculation of target transform is provided via component  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	TWeakObjectPtr<const AActor> AvatarActor;

	bool bCacheForwardOffset = false;
	FVector CachedForwardOffset = FVector::ZeroVector;
	FVector CachedRightOffset = FVector::ZeroVector;
	FVector CachedUpOffset = FVector::ZeroVector;

	bool bWarpingPaused = false;
	bool bRootMotionPaused = false;

	FMotionWarpingTarget()
		: Name(NAME_None), Location(FVector::ZeroVector), Rotation(FRotator::ZeroRotator), Component(nullptr), BoneName(NAME_None), bFollowComponent(false) {}

	FMotionWarpingTarget(const FName& InName, const FTransform& InTransform)
		: Name(InName), Location(InTransform.GetLocation()), Rotation(InTransform.Rotator()), Component(nullptr), BoneName(NAME_None), bFollowComponent(false) {}

	UE_API FMotionWarpingTarget(const FName& InName, const USceneComponent* InComp, FName InBoneName, bool bInbFollowComponent, EWarpTargetLocationOffsetDirection InLocationOffsetDirection = EWarpTargetLocationOffsetDirection::TargetsForwardVector, const AActor* InAvatarActor = nullptr, const FVector& InLocOffset = FVector::ZeroVector, const FRotator& InRotOffset = FRotator::ZeroRotator);
	
	// @TODO: Deprecate this.
	UE_API FMotionWarpingTarget(const FName& InName, const USceneComponent* InComp, FName InBoneName, bool bInbFollowComponent, const FVector& InLocOffset = FVector::ZeroVector, const FRotator& InRotOffset = FRotator::ZeroRotator);

	UE_API FTransform GetTargetTrasform() const;
	UE_API void CacheOffset(const FTransform& InTransform);
	UE_API void RecalculateOffset(FTransform& InTransform) const;

	inline FVector GetLocation() const { return GetTargetTrasform().GetLocation(); }
	inline FQuat GetRotation() const { return GetTargetTrasform().GetRotation(); }
	inline FRotator Rotator() const { return GetTargetTrasform().Rotator(); }

	inline bool operator==(const FMotionWarpingTarget& Other) const
	{
		return Other.Name == Name && Other.Location.Equals(Location) && Other.Rotation.Equals(Rotation) && Other.Component == Component && Other.BoneName == BoneName && Other.bFollowComponent == bFollowComponent && 
			Other.LocationOffsetDirection == LocationOffsetDirection && Other.LocationOffset.Equals(LocationOffset) && Other.RotationOffset.Equals(RotationOffset) && Other.AvatarActor.Get() == AvatarActor.Get();
	}

	inline bool operator!=(const FMotionWarpingTarget& Other) const
	{
		return Other.Name != Name || !Other.Location.Equals(Location) || !Other.Rotation.Equals(Rotation) || Other.Component != Component || Other.BoneName != BoneName || Other.bFollowComponent != bFollowComponent || 
			Other.LocationOffsetDirection != LocationOffsetDirection || !Other.LocationOffset.Equals(LocationOffset) || !Other.RotationOffset.Equals(RotationOffset) || Other.AvatarActor.Get() != AvatarActor.Get();
	}

	static UE_API FTransform GetTargetTransformFromComponent(const USceneComponent* Comp, const FName& BoneName);

};

UENUM(BlueprintType)
enum class EMotionWarpRotationType : uint8
{
	/** Character rotates to match the rotation of the sync point */
	Default,

	/** Character rotates to face the sync point */
	Facing,

	/** Character rotates to match the rotation of the sync point, rotated by half of full angle around up axis (180 degrees yaw) */
	OppositeDefault UE_DEPRECATED(5.8, "Use AdditionalRotationOffset instead") UMETA(Hidden),

	/** Character rotates to face the sync point with their back. Essentially facing + 180 degrees yaw. */
	OppositeFacing UE_DEPRECATED(5.8, "Use AdditionalRotationOffset instead") UMETA(Hidden),
};

UENUM(BlueprintType)
enum class EMotionWarpRotationMethod : uint8
{
	/** Rotate with spherical linear interpolation */
	Slerp, 
	/** Rotate with spherical linear interpolation, not exceeding a max rotation rate*/
	SlerpWithClampedRate,
	/** Rotate with a constant rotation rate*/
	ConstantRate,
	/** Scale rotation in a way that by the end of the anim notify state the owner's rotation matches the warp target's rotation */
	Scale,
};

/** Method used to extract the warp point from the animation */
UENUM(BlueprintType)
enum class EWarpPointAnimProvider : uint8
{
	/** No warp point is provided */
	None,

	/** Warp point defined by a 'hard-coded' transform  user can enter through the warping notify */
	Static,

	/** Warp point defined by a bone */
	Bone
};

UCLASS(MinimalAPI, Abstract)
class URootMotionModifier_Warp : public URootMotionModifier
{
	GENERATED_BODY()

public:

	// UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	// End of UObject

	/** Name used to find the warp target for this modifier */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Config", meta = (ExposeOnSpawn))
	FName WarpTargetName = NAME_None;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Config")
	EWarpPointAnimProvider WarpPointAnimProvider = EWarpPointAnimProvider::None;

	//@TODO: Hide from the UI when Target != Static
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Config", meta = (EditCondition = "WarpPointAnimProvider == EWarpPointAnimProvider::Static"))
	FTransform WarpPointAnimTransform = FTransform::Identity;

	//@TODO: Hide from the UI when Target != Bone
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Config", meta = (EditCondition = "WarpPointAnimProvider == EWarpPointAnimProvider::Bone"))
	FName WarpPointAnimBoneName = NAME_None;

	/** Whether to warp the translation component of the root motion */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Config")
	bool bWarpTranslation = true;

	/** Whether to ignore the Z component of the translation. Z motion will remain untouched */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Config", meta = (EditCondition = "bWarpTranslation"))
	bool bIgnoreZAxis = true;

	/** Whether we warp the actors location or their foot location to the warp target */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Config", meta = (EditCondition = "bWarpTranslation"))
	bool bWarpToFeetLocation = true;

	/** Easing function used when adding translation. Only relevant when there is no translation in the animation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Config", meta = (EditCondition = "bWarpTranslation"))
	EAlphaBlendOption AddTranslationEasingFunc = UE::Anim::DefaultBlendOption;

	/** Custom curve used to add translation when there is none to warp. Only relevant when AddTranslationEasingFunc is set to Custom */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Blend", meta = (EditCondition = "AddTranslationEasingFunc==EAlphaBlendOption::Custom", EditConditionHides))
	TObjectPtr<class UCurveFloat> AddTranslationEasingCurve = nullptr;

	/** Whether to warp the rotation component of the root motion */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Config")
	bool bWarpRotation = true;

	/** Whether rotation should be warp to match the rotation of the sync point or to face the sync point */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Config", meta = (EditCondition = "bWarpRotation"))
	EMotionWarpRotationType RotationType = EMotionWarpRotationType::Default;
	
	/** The method of rotation to use */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Config", meta = (EditCondition = "bWarpRotation"))
	EMotionWarpRotationMethod RotationMethod = EMotionWarpRotationMethod::Slerp;

	/** Should root motion after this notify state be subtracted from warp target transform. 
	This means actor will arrive at warp target transform when the animation finishes, not when the notify ends */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Config")
	bool bSubtractRemainingRootMotion = false;

	/** Applied as a final rotational offset on the warp target */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Config")
	FRotator AdditionalRotationOffset = FRotator::ZeroRotator;

	/**
	 * Allow to modify how fast the rotation is warped.
	 * e.g if the window duration is 2sec and this is 0.5, the target rotation will be reached in 1sec instead of 2sec
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Config", meta = (EditCondition = "RotationMethod!=EMotionWarpRotationMethod::ConstantRate && RotationMethod!=EMotionWarpRotationMethod::Scale && bWarpRotation"))
	float WarpRotationTimeMultiplier = 1.f;

	/** Maximum rotation rate in degrees/sec. Will be the value used in constant rotation rate*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Config", meta = (EditCondition = "RotationMethod!=EMotionWarpRotationMethod::Slerp && bWarpRotation"))
	float WarpMaxRotationRate = 0.f;

	UE_API URootMotionModifier_Warp(const FObjectInitializer& ObjectInitializer);

	//~ Begin FRootMotionModifier Interface
	UE_API virtual void Update(const FMotionWarpingUpdateContext& Context) override;
	//~ End FRootMotionModifier Interface

	/** Event called during update if the target transform changes while the warping is active */
	UE_API virtual void OnTargetTransformChanged();

	UE_API virtual void OnStateChanged(ERootMotionModifierState LastState) override;

	UE_API virtual TSharedPtr<FRootMotionModifierSwapState> GetSwapState() const override;
	UE_API virtual void PopulateSwapState(TSharedPtr<FRootMotionModifierSwapState> InSwapState) const override;
	UE_API virtual void RestoreFromSwapState(TSharedPtr<FRootMotionModifierSwapState> InSwapState) override;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	UE_API void PrintLog(const FString& Name, const FTransform& OriginalRootMotion, const FTransform& WarpedRootMotion) const;
#endif

	inline FVector GetTargetLocation() const { return CachedTargetTransform.GetLocation(); }
	inline FRotator GetTargetRotator() const { return GetTargetRotation().Rotator(); }
	UE_API FQuat GetTargetRotation() const;

	UE_API FQuat WarpRotation(const FTransform& RootMotionDelta, const FTransform& RootMotionTotal, float DeltaSeconds);

protected:

	UPROPERTY()
	FTransform CachedTargetTransform = FTransform::Identity;

	UPROPERTY()
	FTransform RootMotionRemainingAfterNotify = FTransform::Identity;

	/** Cached of the offset from the warp target. Used to calculate the final target transform when a warp target is defined in the animation */
	TOptional<FTransform> CachedOffsetFromWarpPoint;

	/** Rotation offset defined in warp target */
	FQuat RotationOffset = FQuat::Identity;
	
	bool bWarpingPaused = false;
	bool bRootMotionPaused = false;
};

struct FRootMotionModifier_WarpSwapState : public FRootMotionModifierSwapState
{
	FRotator AdditionalRotationOffset = FRotator::ZeroRotator;
	FTransform CachedTargetTransform = FTransform::Identity;
	FTransform RootMotionRemainingAfterNotify = FTransform::Identity;
	TOptional<FTransform> CachedOffsetFromWarpPoint;
	FQuat RotationOffset = FQuat::Identity;
	bool bWarpingPaused = false;
	bool bRootMotionPaused = false;
};

// URootMotionModifier_SimpleWarp. 
// DEPRECATED in favor of URootMotionModifier_SkewWarp (kept for reference)
///////////////////////////////////////////////////////////////

UCLASS(MinimalAPI, Deprecated, meta = (DisplayName = "Simple Warp"))
class UDEPRECATED_RootMotionModifier_SimpleWarp : public URootMotionModifier_Warp
{
	GENERATED_BODY()

public:

	UE_API UDEPRECATED_RootMotionModifier_SimpleWarp(const FObjectInitializer& ObjectInitializer);
	UE_API virtual FTransform ProcessRootMotion(const FTransform& InRootMotion, float DeltaSeconds) override;
};

// URootMotionModifier_Scale
///////////////////////////////////////////////////////////////

UCLASS(MinimalAPI, meta = (DisplayName = "Scale"))
class URootMotionModifier_Scale : public URootMotionModifier
{
	GENERATED_BODY()

public:

	/** Vector used to scale each component of the translation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Config")
	FVector Scale = FVector(1.f);

	URootMotionModifier_Scale(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer) {}

	virtual FTransform ProcessRootMotion(const FTransform& InRootMotion, float DeltaSeconds) override
	{
		FTransform FinalRootMotion = InRootMotion;
		FinalRootMotion.ScaleTranslation(Scale);
		return FinalRootMotion;
	}

	UFUNCTION(BlueprintCallable, Category = "Motion Warping")
	static UE_API URootMotionModifier_Scale* AddRootMotionModifierScale(
		UPARAM(DisplayName = "Motion Warping Comp") UMotionWarpingComponent* InMotionWarpingComp,
		UPARAM(DisplayName = "Animation") const UAnimSequenceBase* InAnimation,
		UPARAM(DisplayName = "Start Time") float InStartTime,
		UPARAM(DisplayName = "End Time") float InEndTime,
		UPARAM(DisplayName = "Scale") FVector InScale = FVector(1.f));
};

// URootMotionModifier_Blueprintable
///////////////////////////////////////////////////////////////

UCLASS(Experimental, MinimalAPI, Blueprintable, meta = (DisplayName = "Blueprintable"))
class URootMotionModifier_Blueprintable : public URootMotionModifier
{
	GENERATED_BODY()

public:

	URootMotionModifier_Blueprintable(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer) {}

	virtual FTransform ProcessRootMotion(const FTransform& InRootMotion, float DeltaSeconds) override;

	UFUNCTION(BlueprintPure, BlueprintImplementableEvent, meta=(DisplayName = "Process Root Motion"), Category = "Settings")
	UE_API FTransform BP_ProcessRootMotion(const AActor* Actor, const USkeletalMeshComponent* Mesh, const FTransform& InRootMotion, float DeltaSeconds) const;
};

#undef UE_API
