// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimSequence.h"
#include "RootMotionModifier.h"
#include "RootMotionModifier_AdjustmentBlendWarp.generated.h"

#define UE_API MOTIONWARPING_API

class ACharacter;
template <class PoseType> struct FCSPose;

USTRUCT()
struct FMotionDeltaTrack
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FTransform> BoneTransformTrack;

	UPROPERTY()
	TArray<FVector> DeltaTranslationTrack;

	UPROPERTY()
	TArray<FRotator> DeltaRotationTrack;

	UPROPERTY()
	FVector TotalTranslation = FVector::ZeroVector;

	UPROPERTY()
	FRotator TotalRotation = FRotator::ZeroRotator;
};

USTRUCT()
struct FMotionDeltaTrackContainer
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FMotionDeltaTrack> Tracks;

	void Init(int32 InNumTracks)
	{
		Tracks.Reserve(InNumTracks);
	}
};

// EXPERIMENTAL: Marked with 'hidedropdown' to prevent it from showing up in the UI since it is not ready for production.
UCLASS(MinimalAPI, hidedropdown, meta = (DisplayName = "Adjustment Blend Warp"))
class URootMotionModifier_AdjustmentBlendWarp : public URootMotionModifier_Warp
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Config")
	bool bWarpIKBones = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Config")
	TArray<FName> IKBones;

	UE_API URootMotionModifier_AdjustmentBlendWarp(const FObjectInitializer& ObjectInitializer);

	UE_API virtual void OnTargetTransformChanged() override;
	UE_API virtual FTransform ProcessRootMotion(const FTransform& InRootMotion, float DeltaSeconds) override;

	UE_API void GetIKBoneTransformAndAlpha(FName BoneName, FTransform& OutTransform, float& OutAlpha) const;

	//UFUNCTION(BlueprintCallable, Category = "Motion Warping")
	static UE_API URootMotionModifier_AdjustmentBlendWarp* AddRootMotionModifierAdjustmentBlendWarp(UMotionWarpingComponent* InMotionWarpingComp, const UAnimSequenceBase* InAnimation, float InStartTime, float InEndTime,
		FName InWarpTargetName, EWarpPointAnimProvider InWarpPointAnimProvider, FTransform InWarpPointAnimTransform, FName InWarpPointAnimBoneName,
		bool bInWarpTranslation, bool bInIgnoreZAxis, bool bInWarpRotation, bool bInWarpIKBones, const TArray<FName>& InIKBones);

	//UFUNCTION(BlueprintPure, Category = "Motion Warping")
	static UE_API void GetAdjustmentBlendIKBoneTransformAndAlpha(ACharacter* Character, FName BoneName, FTransform& OutTransform, float& OutAlpha);

protected:

	UPROPERTY()
	FTransform CachedMeshTransform;

	UPROPERTY()
	FTransform CachedMeshRelativeTransform;

	UPROPERTY()
	FTransform CachedRootMotion;

	UPROPERTY()
	FAnimSequenceTrackContainer Result;

	UE_API void PrecomputeWarpedTracks();

	UE_API FTransform ExtractWarpedRootMotion() const;

	UE_API void ExtractBoneTransformAtTime(FTransform& OutTransform, const FName& BoneName, float Time) const;
	UE_API void ExtractBoneTransformAtTime(FTransform& OutTransform, int32 TrackIndex, float Time) const;
	UE_API void ExtractBoneTransformAtFrame(FTransform& OutTransform, int32 TrackIndex, int32 Frame) const;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	UE_API void DrawDebugWarpedTracks(float DrawDuration) const;
#endif

	static UE_API void ExtractMotionDeltaFromRange(const FBoneContainer& BoneContainer, const UAnimSequenceBase* Animation, float StartTime, float EndTime, float SampleRate, FMotionDeltaTrackContainer& OutMotionDeltaTracks);

	static UE_API void AdjustmentBlendWarp(const FBoneContainer& BoneContainer, const FCSPose<FCompactPose>& AdditivePose, const FMotionDeltaTrackContainer& MotionDeltaTracks, FAnimSequenceTrackContainer& Output);
};

#undef UE_API
