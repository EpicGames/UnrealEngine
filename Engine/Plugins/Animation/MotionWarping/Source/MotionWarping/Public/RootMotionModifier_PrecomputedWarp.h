// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RootMotionModifier.h"
#include "Kismet/KismetMathLibrary.h"
#include "RootMotionModifier_PrecomputedWarp.generated.h"

UENUM(Experimental)
enum class EPrecomputedWarpWeightCurveType
{
	FromRootMotionTranslation,
	FromRootMotionRotation,
	Linear,
	EaseIn,
	EaseOut,
	EaseInOut,
	Instant,
	DoNotWarp,
};

USTRUCT(Experimental)
struct FPrecomputedWarpCurve
{
	GENERATED_BODY()
	
	// Type of curve to interpolate using
	UPROPERTY(EditAnywhere, Category = Properties)
	EPrecomputedWarpWeightCurveType CurveType = EPrecomputedWarpWeightCurveType::FromRootMotionTranslation;

	// Time to start warping, as a ratio of the duration of the Notify
	UPROPERTY(EditAnywhere, Category = "Properties", meta = (UIMin = "0.0", UIMax = "1.0", SliderExponent = "1.0"))
	float StartRatio = 0.f;

	// Time to complete warping, as a ratio of the duration of the Notify
	UPROPERTY(EditAnywhere, Category = "Properties", meta = (UIMin = "0.0", UIMax = "1.0", SliderExponent = "1.0"))
	float EndRatio = 1.f;
};

USTRUCT(Experimental)
struct FPrecomputedWarpSteeringSettings
{
	GENERATED_BODY()

	// Enable smoothing of the steering target orientation, to avoid instant orientation changes
	UPROPERTY(EditAnywhere, Category=Properties)
	bool bEnableSmoothing = true;

	// Sprint stiffness for smoothing
	UPROPERTY(EditAnywhere, Category=Propeties, meta = (EditCondition=bEnableSmoothing))
	float SmoothStiffness = 300.f;

	// Spring damping factor for smoothing
	UPROPERTY(EditAnywhere, Category=Properties, meta = (EditCondition=bEnableSmoothing))
	float SmoothDamping = 1.f;
	
	// When the warped movement direction differs from the animated movement direction by more than this threshold, steering will be disabled.
	UPROPERTY(EditAnywhere, Category = Properties)
	float AngleThreshold = 135.f;
};

UENUM(Experimental)
enum class EPrecomputedWarpUpdateMode
{
	// each frame, apply root motion to get to the expected part of the alignment path in wold space
	World,
	// each frame, apply the root motion from the alignment path relative to the previous frame
	Relative,
};

UCLASS(Experimental, Blueprintable, MinimalAPI, meta = (DisplayName = "Precomputed Warp", Tooltip="(Experimental) Precomputed warp computes the full path to alignment on the first frame of execution and then on following frames just applies that root motion.  Only Suitable for stationary targets."))
class URootMotionModifier_PrecomputedWarp : public URootMotionModifier_Warp
{
	GENERATED_BODY()

public:
	struct FAnimTrajectoryData
	{
		FTransform TargetTransform;
		TArray<FTransform> Trajectory;
		TArray<float> TranslationCurve;
		TArray<float> RotationCurve;
	};

	MOTIONWARPING_API URootMotionModifier_PrecomputedWarp(const FObjectInitializer& ObjectInitializer);

	MOTIONWARPING_API virtual FTransform ProcessRootMotion(const FTransform& InRootMotion, float DeltaSeconds) override;

	MOTIONWARPING_API virtual TSharedPtr<FRootMotionModifierSwapState> GetSwapState() const override;
	MOTIONWARPING_API virtual void PopulateSwapState(TSharedPtr<FRootMotionModifierSwapState> InSwapState) const override;
	MOTIONWARPING_API virtual void RestoreFromSwapState(TSharedPtr<FRootMotionModifierSwapState> InSwapState) override;

	UPROPERTY(EditAnywhere, Category = Blending, Meta = (EditCondition="!bSeparateTranslationCurves"))
	FPrecomputedWarpCurve TranslationWarpingCurve;

	
	UPROPERTY(EditAnywhere, Category = Blending)
	bool bSeparateTranslationCurves = false;

	// curve for warping translation in direction of movement
	UPROPERTY(EditAnywhere, Category = Blending, Meta = (EditCondition="bSeparateTranslationCurves"))
	FPrecomputedWarpCurve TranslationWarpingCurve_InMovementDirection;
	UPROPERTY(EditAnywhere, Category = Blending, Meta = (EditCondition="bSeparateTranslationCurves"))
	FPrecomputedWarpCurve TranslationWarpingCurve_OutOfMovementDirection;
	
	UPROPERTY(EditAnywhere, Category = Blending)
	FPrecomputedWarpCurve RotationWarpingCurve;
	
	// Offset from Root (or AlignBone) to Align to the target transform
	UPROPERTY(EditAnywhere, Category = TargetTransform)
	FTransform AlignOffset;

	// only allow Yaw in the target transform (remove other rotations before warping)
	UPROPERTY(EditAnywhere, Category = TargetTransform)
	bool bForceTargetTransformUpright = false;
	

#if WITH_EDITOR
	// For automating setting the AlignOffset at the end of the NotifyState
	UFUNCTION(CallInEditor, BlueprintCallable, Category = "TechAnimUtils")
	void SetAlignOffset(FTransform NewTransform)
	{
		AlignOffset = NewTransform;
	}
#endif

	UPROPERTY(EditAnywhere, Category = TargetTransform)
	EPrecomputedWarpUpdateMode UpdateMode;
	
	// Bool variable which will disable this notify
	UPROPERTY(EditAnywhere, Category = TargetTransform)
	FName Disable;

	// Steering will rotate the character based on the difference between the animated movement direction and the warped movement direction, to keep the character facing their direction of movement
	UPROPERTY(EditAnywhere, Category = Steering)
	bool bEnableSteering = false;
	
	UPROPERTY(EditAnywhere, Category = Steering, meta=(EditCondition=bEnableSteering))
	FPrecomputedWarpSteeringSettings SteeringSettings;

	// Instance Data

	float GetWeight(float CurrentTime, const FPrecomputedWarpCurve& WarpCurve) const;

	bool bFirstFrame = true;
	float ActualStartTime = 0.f;
	float RoundedEndTime = 0.f;
	float PreviousFrame = 0.f;

	FTransform StartingRootTransform;
	FTransform TargetTransform;
	
	FQuat FilteredSteeringTarget = FQuat::Identity;
	FQuaternionSpringState TargetSmoothingState;

	TArray<FTransform> WarpedTrajectory;

	FAnimTrajectoryData AnimTrajectoryData;
};

struct FRootMotionModifier_PrecomputedWarpSwapState : public FRootMotionModifier_WarpSwapState
{
	bool bFirstFrame = true;
	float ActualStartTime = 0.f;
	float RoundedEndTime = 0.f;
	float PreviousFrame = 0.f;

	FTransform StartingRootTransform;
	FTransform TargetTransform;

	FQuat FilteredSteeringTarget = FQuat::Identity;
	FQuaternionSpringState TargetSmoothingState;

	TArray<FTransform> WarpedTrajectory;

	URootMotionModifier_PrecomputedWarp::FAnimTrajectoryData AnimTrajectoryData;
};
