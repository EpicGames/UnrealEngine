// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RootMotionModifier.h"
#include "RootMotionModifier_SkewWarp.generated.h"

#define UE_API MOTIONWARPING_API

UCLASS(MinimalAPI, meta = (DisplayName = "Skew Warp"))
class URootMotionModifier_SkewWarp : public URootMotionModifier_Warp
{
	GENERATED_BODY()

public:

	UE_API URootMotionModifier_SkewWarp(const FObjectInitializer& ObjectInitializer);

	UE_API virtual FTransform ProcessRootMotion(const FTransform& InRootMotion, float DeltaSeconds) override;
	
	static UE_API FVector WarpTranslation(const FTransform& CurrentTransform, const FVector& DeltaTranslation, const FVector& TotalTranslation, const FVector& TargetLocation);

#if WITH_EDITOR	
	UE_API virtual void DrawInEditor(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* MeshComp, const UAnimSequenceBase* Animation, const FAnimNotifyEvent& NotifyEvent) const override;
	UE_API virtual void DrawCanvasInEditor(FCanvas& Canvas, FSceneView& View, USkeletalMeshComponent* MeshComp, const UAnimSequenceBase* Animation, const FAnimNotifyEvent& NotifyEvent) const override;
	UE_API FTransform GetDebugWarpPointTransform(USkeletalMeshComponent* MeshComp, const UAnimSequenceBase* InAnimation, const UMirrorDataTable* MirrorTable, const float NotifyEndTime) const;
#endif	

	UFUNCTION(BlueprintCallable, Category = "Motion Warping")
	static UE_API URootMotionModifier_SkewWarp* AddRootMotionModifierSkewWarp(
		UPARAM(DisplayName = "Motion Warping Comp") UMotionWarpingComponent* InMotionWarpingComp,
		UPARAM(DisplayName = "Animation") const UAnimSequenceBase* InAnimation,
		UPARAM(DisplayName = "Start Time") float InStartTime,
		UPARAM(DisplayName = "End Time") float InEndTime,
		UPARAM(DisplayName = "Warp Target Name") FName InWarpTargetName,
		UPARAM(DisplayName = "Warp Point Anim Provider") EWarpPointAnimProvider InWarpPointAnimProvider,
		UPARAM(DisplayName = "Warp Point Anim Transform") FTransform InWarpPointAnimTransform,
		UPARAM(DisplayName = "Warp Point Anim Bone Name") FName InWarpPointAnimBoneName,
		UPARAM(DisplayName = "Warp Translation") bool bInWarpTranslation = true,
		UPARAM(DisplayName = "Ignore Z Axis") bool bInIgnoreZAxis = true,
		UPARAM(DisplayName = "Warp Rotation") bool bInWarpRotation = true,
		UPARAM(DisplayName = "Rotation Type") EMotionWarpRotationType InRotationType = EMotionWarpRotationType::Default,
		UPARAM(DisplayName = "Rotation Method") EMotionWarpRotationMethod InRotationMethod = EMotionWarpRotationMethod::Slerp,
		UPARAM(DisplayName = "Warp Rotation Time Multiplier") float InWarpRotationTimeMultiplier = 1.f,
		UPARAM(DisplayName = "Warp Max Rotation Rate") float InWarpMaxRotationRate = 0.f);

protected:
	/**
	* Allows to set maximum warp translation speed clamp ratio.
	* Ratio is relative to the original animation translation speed. E.g. if the MaxSpeedClampRatio == 2.0f actor well be moving with maximum 2x speed of the animation.
	* Applied only in cases when animation has root motion translation.
	* Zero treated as no clamping.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Config")
	float MaxSpeedClampRatio = 0.0f;
};

#undef UE_API
